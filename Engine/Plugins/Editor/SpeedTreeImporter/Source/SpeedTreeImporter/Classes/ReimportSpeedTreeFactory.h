// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// ReimportSpeedTreeFactory
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorReimportHandler.h"
#include "SpeedTreeImportFactory.h"
#include "ReimportSpeedTreeFactory.generated.h"

UCLASS(collapsecategories)
class UReimportSpeedTreeFactory : public USpeedTreeImportFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin FReimportHandler interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	virtual TObjectPtr<UObject>* GetFactoryObject() const override
	{
		return &GCMark;
	}

	mutable TObjectPtr<UObject> GCMark{this};
	//~ End FReimportHandler interface
};
