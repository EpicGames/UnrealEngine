// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Import data and options used when importing a static mesh from fbx
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorFramework/AssetImportData.h"
#include "Types/SlateEnums.h"
#include "Layout/Visibility.h"
#include "IDetailCustomization.h"

#include "SpeedTreeImportData.generated.h"



class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;
class SWidget;


/** Geometry import type */
UENUM()
enum EImportGeometryType : int
{
	IGT_3D UMETA(DisplayName = "3D LODs"),
	IGT_Billboards UMETA(DisplayName = "Billboards"),
	IGT_Both UMETA(DisplayName = "Both")
};


/** LOD type **/
UENUM()
enum EImportLODType : int
{
	ILT_PaintedFoliage UMETA(DisplayName = "Painted Foliage"),
	ILT_IndividualActors UMETA(DisplayName = "Individual Actors")
};


UCLASS(BlueprintType, config=EditorPerProjectUserSettings, perObjectConfig, MinimalAPI)
class USpeedTreeImportData : public UAssetImportData
{
	GENERATED_UCLASS_BODY()

	/** Specify the tree scale */
	UPROPERTY(EditAnywhere, config, Category=Mesh, meta=(DisplayName = "Tree Scale"))
	float TreeScale;

	/** Choose whether to import as a 3D asset, billboard or both */
	UPROPERTY(EditAnywhere, config, Category=Mesh, meta = (DisplayName = "Geometry"))
	TEnumAsByte<enum EImportGeometryType> ImportGeometryType;

	/** Choose whether painted foliage or individual actor */
	UPROPERTY(EditAnywhere, config, Category=Mesh, meta = (DisplayName = "LOD Setup", SpeedTreeVersion = "8"))
	TEnumAsByte<enum EImportLODType> LODType;

	/**  */
	UPROPERTY(EditAnywhere, config, Category=Mesh, meta = (DisplayName = "Include Collision", SpeedTreeVersion = "8"))
	uint32 IncludeCollision : 1;

	/**  */
	UPROPERTY(EditAnywhere, config, Category=Materials, meta = (DisplayName = "Create Materials", SpeedTreeVersion = "8"))
	uint32 MakeMaterialsCheck : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include Normal Maps"))
	uint32 IncludeNormalMapCheck : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include Detail Maps"))
	uint32 IncludeDetailMapCheck : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include Specular Maps"))
	uint32 IncludeSpecularMapCheck : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include Branch Seam Smoothing"))
	uint32 IncludeBranchSeamSmoothing : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include SpeedTree AO"))
	uint32 IncludeSpeedTreeAO : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include Random Color Variation"))
	uint32 IncludeColorAdjustment : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include Subsurface", SpeedTreeVersion = "8"))
	uint32 IncludeSubsurface : 1;

	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "MakeMaterialsCheck", DisplayName = "Include Vertex Processing", SpeedTreeVersion = "8"))
	uint32 IncludeVertexProcessingCheck : 1;

	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "IncludeVertexProcessingCheck", DisplayName = "Include Wind"))
	uint32 IncludeWindCheck : 1;
	
	/**  */
	UPROPERTY(EditAnywhere, config, Category = Materials, meta = (EditCondition = "IncludeVertexProcessingCheck", DisplayName = "Include Smooth LOD"))
	uint32 IncludeSmoothLODCheck : 1;

	void CopyFrom(USpeedTreeImportData* Other);
};



class FSpeedTreeImportDataDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	FSpeedTreeImportDataDetails();

	void OnForceRefresh();

	USpeedTreeImportData *SpeedTreeImportData;
	class IDetailLayoutBuilder* CachedDetailBuilder;
};

