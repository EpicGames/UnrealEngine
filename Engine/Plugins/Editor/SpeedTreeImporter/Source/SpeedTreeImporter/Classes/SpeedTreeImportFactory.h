// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Factory for importing SpeedTrees
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "SpeedTreeImportFactory.generated.h"

class UMaterialInterface;
class USpeedTreeImportData;

UCLASS(BlueprintType, hidecategories=Object)
class USpeedTreeImportFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
#if WITH_SPEEDTREE
	virtual bool DoesSupportClass(UClass * Class) override;
	virtual UClass* ResolveSupportedClass() override;
	virtual UObject* FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;

	UObject* FactoryCreateBinary7(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled);
	UObject* FactoryCreateBinary8(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled);
	UObject* FactoryCreateBinary9(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled);

private:
	USpeedTreeImportData* GetAutomatedImportOptions(USpeedTreeImportData* ExistingImportData) const;

public:
#endif

	TSet<UPackage*> LoadedPackages;
	UMaterialInterface* TemplateMaterial;
	UMaterialInterface* TemplateBillboardMaterial;
};

