// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReimportSpeedTreeFactory.h"
#include "Misc/Paths.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/StaticMesh.h"
#include "Factories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReimportSpeedTreeFactory)



#define LOCTEXT_NAMESPACE "EditorFactories"



UReimportSpeedTreeFactory::UReimportSpeedTreeFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	#if WITH_SPEEDTREE
		SupportedClass = UStaticMesh::StaticClass();
		Formats.Add(TEXT("srt;SpeedTree"));
		Formats.Add(TEXT("st;SpeedTree v8"));
		Formats.Add(TEXT("st9;SpeedTree v9"));
	#endif

	bCreateNew = false;
	bText = false;
}

bool UReimportSpeedTreeFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
#if WITH_SPEEDTREE
	UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
	if (Mesh && Mesh->GetAssetImportData())
	{
		const FString FileExtension = FPaths::GetExtension(Mesh->GetAssetImportData()->GetFirstFilename());
			const bool bIsSpeedTree = (FCString::Stricmp(*FileExtension, TEXT("SRT")) == 0) ||
										(FCString::Stricmp(*FileExtension, TEXT("ST")) == 0) ||
										(FCString::Stricmp(*FileExtension, TEXT("ST9")) == 0);

		if (bIsSpeedTree)
		{
			// SpeedTrees file extension matches with filepath			
			Mesh->GetAssetImportData()->ExtractFilenames(OutFilenames);
			return true;
		}		
	}
#endif // #if WITH_SPEEDTREE
	return false;
}

void UReimportSpeedTreeFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
#if WITH_SPEEDTREE
	UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
	if (Mesh && Mesh->GetAssetImportData() && ensure(NewReimportPaths.Num() == 1))
	{
		Mesh->GetAssetImportData()->UpdateFilenameOnly(NewReimportPaths[0]);
	}
#endif // #if WITH_SPEEDTREE
}

EReimportResult::Type UReimportSpeedTreeFactory::Reimport(UObject* Obj)
{
#if WITH_SPEEDTREE
	UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
	if (!Mesh)
	{
		return EReimportResult::Failed;
	}

	const FString Filename = Mesh->GetAssetImportData()->GetFirstFilename();
	const FString FileExtension = FPaths::GetExtension(Filename);
	const bool bIsSpeedTree = (FCString::Stricmp(*FileExtension, TEXT("SRT")) == 0) ||
								(FCString::Stricmp(*FileExtension, TEXT("ST")) == 0) ||
								(FCString::Stricmp(*FileExtension, TEXT("ST9")) == 0);

	if (!bIsSpeedTree)
	{
		return EReimportResult::Failed;
	}

	if (!(Filename.Len()))
	{
		// Since this is a new system most static meshes don't have paths, so logging has been commented out
		//UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: static mesh resource does not have path stored.");
		return EReimportResult::Failed;
	}

	UE_LOGF(LogEditorFactories, Log, "Performing atomic reimport of [%ls]", *Filename);

	bool OutCanceled = false;

	if (ImportObject(Mesh->GetClass(), Mesh->GetOuter(), *Mesh->GetName(), RF_Public | RF_Standalone, Filename, nullptr, OutCanceled) != nullptr)
	{
		UE_LOGF(LogEditorFactories, Log, "-- imported successfully");

		Mesh->GetAssetImportData()->Update(Filename);
		Mesh->MarkPackageDirty();

		return EReimportResult::Succeeded;
	}
	else if (OutCanceled)
	{
		UE_LOGF(LogEditorFactories, Warning, "-- import canceled");
	}
	else
	{
		UE_LOGF(LogEditorFactories, Warning, "-- import failed");
	}
#endif // #if WITH_SPEEDTREE

	return EReimportResult::Failed;
}

int32 UReimportSpeedTreeFactory::GetPriority() const
{
	return ImportPriority;
}

#undef LOCTEXT_NAMESPACE
