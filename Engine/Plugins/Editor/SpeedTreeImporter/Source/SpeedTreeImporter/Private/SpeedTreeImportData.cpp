// Copyright Epic Games, Inc. All Rights Reserved.
#include "SpeedTreeImportData.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SCheckBox.h"

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectHash.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpeedTreeImportData)

#define LOCTEXT_NAMESPACE "SpeedTreeImportDataDetails"

DEFINE_LOG_CATEGORY_STATIC(LogSpeedTreeImportData, Log, All);

USpeedTreeImportData::USpeedTreeImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ImportGeometryType = IGT_3D;
	TreeScale = 30.48f;
	LODType = ILT_PaintedFoliage;
}

void USpeedTreeImportData::CopyFrom(USpeedTreeImportData* Other)
{
	TreeScale = Other->TreeScale;
	ImportGeometryType = Other->ImportGeometryType;
	LODType = Other->LODType;
	IncludeCollision = Other->IncludeCollision;
	MakeMaterialsCheck = Other->MakeMaterialsCheck;
	IncludeNormalMapCheck = Other->IncludeNormalMapCheck;
	IncludeDetailMapCheck = Other->IncludeDetailMapCheck;
	IncludeSpecularMapCheck = Other->IncludeSpecularMapCheck;
	IncludeBranchSeamSmoothing = Other->IncludeBranchSeamSmoothing;
	IncludeSpeedTreeAO = Other->IncludeSpeedTreeAO;
	IncludeColorAdjustment = Other->IncludeColorAdjustment;
	IncludeSubsurface = Other->IncludeSubsurface;
	IncludeVertexProcessingCheck = Other->IncludeVertexProcessingCheck;
	IncludeWindCheck = Other->IncludeWindCheck;
	IncludeSmoothLODCheck = Other->IncludeSmoothLODCheck;
}

FSpeedTreeImportDataDetails::FSpeedTreeImportDataDetails()
{
	SpeedTreeImportData = nullptr;
	CachedDetailBuilder = nullptr;
}

TSharedRef<IDetailCustomization> FSpeedTreeImportDataDetails::MakeInstance()
{
	return MakeShareable(new FSpeedTreeImportDataDetails);
}

/** IDetailCustomization interface */
void FSpeedTreeImportDataDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	CachedDetailBuilder = &DetailLayout;
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailLayout.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);
	SpeedTreeImportData = Cast<USpeedTreeImportData>(EditingObjects[0].Get());
	if (SpeedTreeImportData == nullptr)
	{
		return;
	}

	// figure out if it is SpeedTree 8 to change what is shown in the dialog
	const FString FileExtension = FPaths::GetExtension(SpeedTreeImportData->GetFirstFilename());
	bool bSpeedTree8 = (FCString::Stricmp(*FileExtension, TEXT("ST")) == 0);

	//We have to hide FilePath category
	DetailLayout.HideCategory(FName(TEXT("File Path")));
	
	//Mesh category Must be the first category (Important)
	DetailLayout.EditCategory(FName(TEXT("Mesh")), FText::GetEmpty(), ECategoryPriority::Important);

	if (bSpeedTree8)
	{
		DetailLayout.HideProperty(FName(TEXT("TreeScale")));
		DetailLayout.HideProperty(FName(TEXT("ImportGeometryType")));
	}

	//Get the Materials category
	IDetailCategoryBuilder& MaterialsCategoryBuilder = DetailLayout.EditCategory(FName(TEXT("Materials")));
	TArray<TSharedRef<IPropertyHandle>> MaterialCategoryDefaultProperties;
	MaterialsCategoryBuilder.GetDefaultProperties(MaterialCategoryDefaultProperties);
	
	//We have to make the logic for vertex processing
	TSharedRef<IPropertyHandle> MakeMaterialsCheckProp = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USpeedTreeImportData, MakeMaterialsCheck));
	MakeMaterialsCheckProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FSpeedTreeImportDataDetails::OnForceRefresh));

	TSharedRef<IPropertyHandle> IncludeVertexProcessingCheckProp = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USpeedTreeImportData, IncludeVertexProcessingCheck));
	IncludeVertexProcessingCheckProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FSpeedTreeImportDataDetails::OnForceRefresh));

	//Hide all properties, we will show them in the correct order with the correct grouping
	for (TSharedRef<IPropertyHandle> Handle : MaterialCategoryDefaultProperties)
	{
		DetailLayout.HideProperty(Handle);
	}

	MaterialsCategoryBuilder.AddProperty(MakeMaterialsCheckProp);
	if (SpeedTreeImportData->MakeMaterialsCheck)
	{
		for (TSharedRef<IPropertyHandle> Handle : MaterialCategoryDefaultProperties)
		{
			// skip any properties that don't match speedtree 8
			const FString& SpeedTreeMetaData = Handle->GetMetaData(TEXT("SpeedTreeVersion"));
			if (bSpeedTree8 && SpeedTreeMetaData.Compare(TEXT("8")) != 0)
			{
				continue;
			}

			const FString& MetaData = Handle->GetMetaData(TEXT("EditCondition"));
			if (MetaData.Compare(TEXT("MakeMaterialsCheck")) == 0 && IncludeVertexProcessingCheckProp->GetProperty() != Handle->GetProperty())
			{
				MaterialsCategoryBuilder.AddProperty(Handle);
			}
		}

		if (bSpeedTree8)
		{
			MaterialsCategoryBuilder.AddProperty(IncludeVertexProcessingCheckProp);
		}
		else
		{
			IDetailGroup& VertexProcessingGroup = MaterialsCategoryBuilder.AddGroup(FName(TEXT("VertexProcessingGroup")), LOCTEXT("VertexProcessingGroup_DisplayName", "Vertex Processing"), false, true);
			VertexProcessingGroup.AddPropertyRow(IncludeVertexProcessingCheckProp);
			for (TSharedRef<IPropertyHandle> Handle : MaterialCategoryDefaultProperties)
			{
				const FString& MetaData = Handle->GetMetaData(TEXT("EditCondition"));
				if (MetaData.Compare(TEXT("IncludeVertexProcessingCheck")) == 0)
				{
					VertexProcessingGroup.AddPropertyRow(Handle);
				}
			}
		}
	}
}

void FSpeedTreeImportDataDetails::OnForceRefresh()
{
	if (CachedDetailBuilder)
	{
		CachedDetailBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
