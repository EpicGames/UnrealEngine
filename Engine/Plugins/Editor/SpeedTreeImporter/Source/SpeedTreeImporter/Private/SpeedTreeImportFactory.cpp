// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpeedTreeImportFactory.h"

#include "AssetImportTask.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "EditorReimportHandler.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Factories/MaterialFactoryNew.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Factories/TextureFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/Reply.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInterface.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "RenderUtils.h"
#include "SlateOptMacros.h"
#include "StaticParameterSet.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/GCObject.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#include "Materials/MaterialExpressionClamp.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionConstantBiasScale.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionSpeedTree.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTwoSidedSign.h"
#include "Materials/MaterialExpressionVertexColor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ComponentReregisterContext.h"
#include "Editor/UnrealEd/Private/GeomFitUtils.h"
#include "Interfaces/IMainFrameModule.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "SpeedTreeWind.h"
#include "StaticMeshAttributes.h"
#include "RenderMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpeedTreeImportFactory)

#if WITH_SPEEDTREE

THIRD_PARTY_INCLUDES_START
#include "Core/Core.h"
#include "TreeReader8.h"
#include "TreeReader9.h"
THIRD_PARTY_INCLUDES_END

#endif // WITH_SPEEDTREE
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/BodySetup.h"

#include "PropertyEditorModule.h"
#include "IDetailsView.h"

#include "SpeedTreeImportData.h"

using namespace SpeedTreeDataBuffer;

#define LOCTEXT_NAMESPACE "SpeedTreeImportFactory"

DEFINE_LOG_CATEGORY_STATIC(LogSpeedTreeImport, Log, All);


namespace UE
{
namespace SpeedTreeImporter
{
namespace Private
{
	TArray<FStaticMaterial> ClearOutOldMesh(UStaticMesh& Mesh)
	{
		TArray<FStaticMaterial> OldMaterials;
	
		OldMaterials = Mesh.GetStaticMaterials();
		UMaterialInterface* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		for (int32 i = 0; i < OldMaterials.Num(); ++i)
		{
			UMaterialInterface* MaterialInterface = OldMaterials[i].MaterialInterface;
			if(MaterialInterface && MaterialInterface != DefaultMaterial)
			{
				MaterialInterface->PreEditChange(NULL);
				MaterialInterface->PostEditChange();
			}
		}

		// Free any RHI resources for existing mesh before we re-create in place.
		Mesh.PreEditChange(NULL);

		return OldMaterials;
	}
}
}
}

/** UI to pick options when importing  SpeedTree */
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
class SSpeedTreeImportOptions : public SCompoundWidget
{
public:
	USpeedTreeImportData *SpeedTreeImportData;

	/** Whether we should go ahead with import */
	bool							bImport;

	/** Window that owns us */
	TSharedPtr<SWindow>				WidgetWindow;

	TSharedPtr<IDetailsView> DetailsView;

public:
	SLATE_BEGIN_ARGS(SSpeedTreeImportOptions) 
		: _WidgetWindow()
		, _ReimportAssetData(nullptr)
		{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(USpeedTreeImportData*, ReimportAssetData)
	SLATE_END_ARGS()

	SSpeedTreeImportOptions() :
		bImport(false)
	{
		DetailsView = nullptr;
		
		// The name options is important for the config storage
		FName SpeedTreeImportDataName = TEXT("SpeedTreeImportData");
		if (UObject* ObjectToMove = FindObjectFast<UObject>(GetTransientPackage(), SpeedTreeImportDataName))
		{
			ObjectToMove->Rename();
		}
		SpeedTreeImportData = NewObject<USpeedTreeImportData>(GetTransientPackage(), SpeedTreeImportDataName);
	}

	void Construct(const FArguments& InArgs)
	{
		WidgetWindow = InArgs._WidgetWindow;
		USpeedTreeImportData* ReimportAssetData = InArgs._ReimportAssetData;

		if (ReimportAssetData != nullptr)
		{
			//If we reimport we have to load the original import options
			//Do not use the real mesh data (ReimportAssetData) in case the user cancel the operation.
			SpeedTreeImportData->CopyFrom(ReimportAssetData);
		}
		else
		{
			//When simply importing we load the local config file of the user so they rerieve the last import options
			SpeedTreeImportData->LoadConfig();
		}

		// set the filename now so the dialog can tell if it is SpeedTree 7 or 8
		SpeedTreeImportData->Update(UFactory::GetCurrentFilename());

		TSharedPtr<SBox> InspectorBox;
		
		// Create widget
		this->ChildSlot
		[
			SNew(SBorder)
			. BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
			. Content()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SAssignNew(InspectorBox, SBox)
					.MaxDesiredHeight(650.0f)
					.WidthOverride(400.0f)
				]
				// Ok/Cancel
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					[
						//Left Button array
						SNew(SUniformGridPanel)
						.SlotPadding(3)
						+ SUniformGridPanel::Slot(0, 0)
						[
							SNew(SButton)
							.Text(LOCTEXT("SpeedTreeOptionWindow_ResetToDefault", "Reset to Default"))
							.OnClicked(this, &SSpeedTreeImportOptions::OnResetToDefault)
						]
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Right)
					[
						//Right button array
						SNew(SUniformGridPanel)
						.SlotPadding(3)
						+SUniformGridPanel::Slot(0,0)
						[
							SNew(SButton)
							.Text(LOCTEXT("SpeedTreeOptionWindow_Import", "Import"))
							.OnClicked(this, &SSpeedTreeImportOptions::OnImport)
						]
						+SUniformGridPanel::Slot(1,0)
						[
							SNew(SButton)
							.Text(LOCTEXT("SpeedTreeOptionWindow_Cancel", "Cancel"))
							.OnClicked(this, &SSpeedTreeImportOptions::OnCancel)
						]
					]
				]
			]
		];
		
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		InspectorBox->SetContent(DetailsView->AsShared());
		DetailsView->SetObject(SpeedTreeImportData);
	}

	/** If we should import */
	bool ShouldImport()
	{
		return bImport;
	}

private:

	/** Called when 'OK' button is pressed */
	FReply OnImport()
	{
		bImport = true;
		WidgetWindow->RequestDestroyWindow();
		SpeedTreeImportData->SaveConfig(CPF_Config, nullptr, GConfig, false);
		return FReply::Handled();
	}

	FReply OnResetToDefault()
	{
		if (DetailsView.IsValid())
		{
			// Reset values from the CDO 
			UClass* Class = SpeedTreeImportData->GetClass();
			UObject* CDO = Class->GetDefaultObject();

			for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
			{
				// Only reset the property that would have been store in the config
				if (Property->HasAnyPropertyFlags(CPF_Config))
				{
					void* Dest = Property->ContainerPtrToValuePtr<void>(SpeedTreeImportData);
					const void* Source = Property->ContainerPtrToValuePtr<void>(CDO);
					Property->CopyCompleteValue(Dest, Source);
				}
			}


			DetailsView->SetObject(SpeedTreeImportData, true);
		}
		return FReply::Handled();
	}

	/** Called when 'Cancel' button is pressed */
	FReply OnCancel()
	{
		bImport = false;
		WidgetWindow->RequestDestroyWindow();
		return FReply::Handled();
	}

	void ScaleTextCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
	{
		TTypeFromString<float>::FromString(SpeedTreeImportData->TreeScale, *(CommentText.ToString()));
	}
};
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/**
 * A small context to avoid importing multiple time the same file
 */
struct FSpeedTreeImportContext : public FGCObject
{
	TMap<FString, TObjectPtr<UMaterialInterface>> ImportedMaterials;
	TMap<FString, TObjectPtr<UTexture>> ImportedTextures;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		for (auto& KV : ImportedMaterials)
		{
			Collector.AddReferencedObject(KV.Value);
		}
		for (auto& KV : ImportedTextures)
		{
			Collector.AddReferencedObject(KV.Value);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSpeedTreeImportContext");
	}
};

//////////////////////////////////////////////////////////////////////////

USpeedTreeImportFactory::USpeedTreeImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = UStaticMesh::StaticClass();

	bEditorImport = true;
	bText = false;

#if WITH_SPEEDTREE
	Formats.Add(TEXT("srt;SpeedTree"));
	Formats.Add(TEXT("st;SpeedTree v8"));
	Formats.Add(TEXT("st9;SpeedTree v9"));
#endif

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> TemplateMaterialFinder(TEXT("/SpeedTreeImporter/SpeedTree9/SpeedTreeMaster"));
	TemplateMaterial = TemplateMaterialFinder.Object;
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> TemplateBillboardMaterialFinder(TEXT("/SpeedTreeImporter/SpeedTree9/SpeedTreeBillboardMaster"));
	TemplateBillboardMaterial = TemplateBillboardMaterialFinder.Object;
}

FText USpeedTreeImportFactory::GetDisplayName() const
{
	return LOCTEXT("SpeedTreeImportFactoryDescription", "SpeedTree");
}

bool USpeedTreeImportFactory::FactoryCanImport(const FString& Filename)
{
	bool bCanImport = false;

#if WITH_SPEEDTREE
	if (FPaths::GetExtension(Filename) == TEXT("srt"))
	{
		// SpeedTree RealTime files should begin with the bytes "SRT " in the header. If they don't, then this isn't a SpeedTree file
		TArray<uint8> FileData;

		FFileHelper::LoadFileToArray(FileData, *Filename);

		if (FileData.Num() > 4)
		{
			bCanImport = (FileData[0] == 'S' &&
						  FileData[1] == 'R' &&
						  FileData[2] == 'T' &&
						  FileData[3] == ' ');
		}
	}
	else if (FPaths::GetExtension(Filename) == TEXT("st"))
	{
		// GameEngine8 files should begin with the token "SpeedTree___"
		TArray<uint8> FileData;

		FFileHelper::LoadFileToArray(FileData, *Filename);

		if (FileData.Num() > 12)
		{
			bCanImport = (FileData[0] == 'S' &&
						  FileData[1] == 'p' &&
						  FileData[2] == 'e' &&
						  FileData[3] == 'e' &&
						  FileData[4] == 'd' &&
						  FileData[5] == 'T' &&
						  FileData[6] == 'r' &&
						  FileData[7] == 'e' &&
						  FileData[8] == 'e' &&
						  FileData[9] == '_' &&
						  FileData[10] == '_' &&
						  FileData[11] == '_');
		}
	}
	else if (FPaths::GetExtension(Filename) == TEXT("st9"))
	{
		// GameEngine8 files should begin with the token "SpeedTree___"
		TArray<uint8> FileData;

		FFileHelper::LoadFileToArray(FileData, *Filename);

		if (FileData.Num() > 16)
		{
			bCanImport = (FileData[0] == 'S' &&
						  FileData[1] == 'p' &&
						  FileData[2] == 'e' &&
						  FileData[3] == 'e' &&
						  FileData[4] == 'd' &&
						  FileData[5] == 'T' &&
  						  FileData[6] == 'r' &&
						  FileData[7] == 'e' &&
						  FileData[8] == 'e' &&
						  FileData[9] == '9' &&
						  FileData[10] == '_' &&
						  FileData[11] == '_' &&
						  FileData[12] == '_' &&
						  FileData[13] == '_' &&
						  FileData[14] == '_' &&
						  FileData[15] == '_');
		}
}

#else	// WITH_SPEEDTREE
	bCanImport = Super::FactoryCanImport(Filename);
#endif	// WITH_SPEEDTREE

	return bCanImport;
}

#if WITH_SPEEDTREE

bool USpeedTreeImportFactory::DoesSupportClass(UClass * Class)
{
	return (Class == UStaticMesh::StaticClass());
}

UClass* USpeedTreeImportFactory::ResolveSupportedClass()
{
	return UStaticMesh::StaticClass();
}

UTexture* CreateSpeedTreeMaterialTexture(UObject* Parent,const FString& Filename, bool bNormalMap, bool bMasks, TSet<UPackage*>& LoadedPackages, FSpeedTreeImportContext& ImportContext)
{
	if (Filename.IsEmpty( ))
	{
		return nullptr;
	}

	if (auto* Texture = ImportContext.ImportedTextures.Find(Filename))
	{
		// The texture was already processed
		return *Texture;
	}

	// set where to place the materials
	FString Extension = FPaths::GetExtension(Filename).ToLower();
	FString TextureName = ObjectTools::SanitizeObjectName(Filename) + TEXT("_Tex");
	FString BasePackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName()) / TextureName;
	BasePackageName = UPackageTools::SanitizePackageName(BasePackageName);

	UTexture* ExistingTexture = nullptr;
	UPackage* Package = nullptr;
	{
		FString ObjectPath = BasePackageName + TEXT(".") + TextureName;
		ExistingTexture = LoadObject<UTexture>(nullptr, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn);
	}

	if (ExistingTexture)
	{
		Package = ExistingTexture->GetOutermost();
	}
	else
	{
		const FString Suffix(TEXT(""));

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FString FinalPackageName;
		AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, Suffix, FinalPackageName, TextureName);

		Package = CreatePackage(*FinalPackageName);
	}

	// try opening from absolute path
	FString FilenamePath = FPaths::GetPath(UFactory::GetCurrentFilename()) + "/" + Filename;
	TArray<uint8> TextureData;
	if (!(FFileHelper::LoadFileToArray(TextureData, *FilenamePath) && TextureData.Num() > 0))
	{
		UE_LOGF(LogSpeedTreeImport, Warning, "Unable to find Texture file %ls", *FilenamePath);
		return nullptr;
	}

	auto TextureFact = NewObject<UTextureFactory>();
	TextureFact->AddToRoot();
	TextureFact->SuppressImportOverwriteDialog();

	if (bNormalMap)
	{
		TextureFact->LODGroup = TEXTUREGROUP_WorldNormalMap;
		TextureFact->CompressionSettings = TC_Normalmap;
	}
		
	if (bMasks)
	{
		TextureFact->CompressionSettings = TC_Masks;
	}

	const uint8* PtrTexture = TextureData.GetData();
	UTexture* Texture = (UTexture*)TextureFact->FactoryCreateBinary(UTexture2D::StaticClass(), Package, *TextureName, RF_Standalone|RF_Public, NULL, *Extension, PtrTexture, PtrTexture + TextureData.Num(), GWarn);
	if (Texture != nullptr)
	{
		if (bMasks)
		{
			if (Texture->SRGB != false || Texture->CompressionSettings != TC_Masks)
			{
				Texture->PreEditChange(nullptr);
				Texture->SRGB = false;
				Texture->CompressionSettings = TC_Masks;
				Texture->PostEditChange();
			}
		}
		Texture->AssetImportData->Update(FilenamePath);

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(Texture);

		// Set the dirty flag so this package will get saved later
		Package->SetDirtyFlag(true);
		LoadedPackages.Add(Package);
	}

	Texture->RemoveFromRoot();

	// Keep track of the created textures
	ImportContext.ImportedTextures.Add(Filename, Texture);

	return Texture;
}

void LayoutMaterial(UMaterialInterface* MaterialInterface, bool bOffsetOddColumns = false)
{
	UMaterial* Material = MaterialInterface->GetMaterial();
	Material->EditorX = 0;
	Material->EditorY = 0;

	const int32 Height = 200;
	const int32 Width = 300;

	// layout X to make sure each input is 1 step further than output connection
	bool bContinue = true;
	while (bContinue)
	{
		bContinue = false;

		for (UMaterialExpression* Expression : Material->GetExpressions())
		{
			Expression->MaterialExpressionEditorX = FMath::Min(Expression->MaterialExpressionEditorX, -Width);

			for (FExpressionInputIterator It{ Expression }; It; ++It)
			{
				if (UMaterialExpression* InputExpression = It->Expression)
				{
					if (InputExpression->MaterialExpressionEditorX > Expression->MaterialExpressionEditorX - Width)
					{
						InputExpression->MaterialExpressionEditorX = Expression->MaterialExpressionEditorX - Width;
						bContinue = true;
					}
				}
			}
		}
	}

	// run through each column of expressions, sort them by outputs, and layout Y
	bContinue = true;
	int32 Column = 1;
	while (bContinue)
	{
		TArray<UMaterialExpression*> ColumnExpressions;
		for (UMaterialExpression* Expression : Material->GetExpressions())
		{
			if (Expression->MaterialExpressionEditorX == -Width * Column)
			{
				Expression->MaterialExpressionEditorY = 0;
				int32 NumOutputs = 0;

				// connections to the material (these are weighted more)
				for (int32 MaterialPropertyIndex = 0; MaterialPropertyIndex < MP_MAX; ++MaterialPropertyIndex)
				{
					FExpressionInput* FirstLevelExpression = Material->GetExpressionInputForProperty(EMaterialProperty(MaterialPropertyIndex));
					if (FirstLevelExpression != NULL && FirstLevelExpression->Expression == Expression)
					{
						Expression->MaterialExpressionEditorY += MaterialPropertyIndex * 25 * 100;
						NumOutputs += 100;
						break;
					}
				}
				
				// all the outputs to other expressions
				for (UMaterialExpression* OtherExpression : Material->GetExpressions())
				{
					for (FExpressionInputIterator It{ OtherExpression }; It; ++It)
					{
						if (It->Expression == Expression)
						{
							Expression->MaterialExpressionEditorY += OtherExpression->MaterialExpressionEditorY;
							++NumOutputs;
						}
					}
				}

				if (NumOutputs == 0)
				{
					Expression->MaterialExpressionEditorY = 10000;
				}
				else
				{
					Expression->MaterialExpressionEditorY /= NumOutputs;
				}

				ColumnExpressions.Add(Expression);
			}
		}

		struct FMaterialExpressionSorter
		{
			bool operator()(const UMaterialExpression& A, const UMaterialExpression& B) const
			{
				return (A.MaterialExpressionEditorY < B.MaterialExpressionEditorY);
			}
		};
		ColumnExpressions.Sort(FMaterialExpressionSorter());

		int32 CurrentHeight = (bOffsetOddColumns && (Column % 2)) ? (Height / 3) : 0;
		for (int32 ExpressionIndex = 0; ExpressionIndex < ColumnExpressions.Num(); ++ExpressionIndex)
		{
			UMaterialExpression* Expression = ColumnExpressions[ExpressionIndex];

			Expression->MaterialExpressionEditorY = CurrentHeight;
			CurrentHeight += Height;
		}

		++Column;
		bContinue = (ColumnExpressions.Num() > 0);
	}
}

UMaterialInterface* CreateSpeedTreeMaterial7(UObject* Parent, FString MaterialFullName, const SpeedTree::SRenderState* RenderState, USpeedTreeImportData* SpeedTreeImportData, ESpeedTreeWindType WindType, int32 NumBillboards, TSet<UPackage*>& LoadedPackages, FSpeedTreeImportContext& ImportContext)
{
	// Make sure we have a parent
	if (!SpeedTreeImportData->MakeMaterialsCheck || !ensure(Parent))
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}
	
	if (auto* Material = ImportContext.ImportedMaterials.Find(MaterialFullName))
	{
		// The material was already imported
		return *Material;
	}

	// set where to place the materials
	FString FixedMaterialName = MaterialFullName + TEXT("_Mat");
	FString NewPackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName()) + TEXT("/") + FixedMaterialName;
	UPackage* Package = UPackageTools::FindOrCreatePackageForAssetType(FName(*NewPackageName), UMaterial::StaticClass());
	check(Package);
	NewPackageName = Package->GetFullName();
	FixedMaterialName = FPaths::GetBaseFilename(NewPackageName, true);

	// does not override existing materials
	UMaterialInterface* UnrealMaterialInterface = FindObject<UMaterialInterface>(Package, *FixedMaterialName);
	if (UnrealMaterialInterface != NULL)
	{
		// Keep track of the processed materials
		ImportContext.ImportedMaterials.Add(MaterialFullName, UnrealMaterialInterface);

		// touch the textures anyway to make sure they reload if necessary
		UTexture* DiffuseTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_DIFFUSE]), false, false, LoadedPackages, ImportContext);
		if (DiffuseTexture)
		{
			if (RenderState->m_bBranchesPresent && SpeedTreeImportData->IncludeDetailMapCheck)
			{
				UTexture* DetailTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_DETAIL_DIFFUSE]), false, false, LoadedPackages, ImportContext);
			}
		}
		if (SpeedTreeImportData->IncludeSpecularMapCheck)
		{
			UTexture* SpecularTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_SPECULAR_MASK]), false, false, LoadedPackages, ImportContext);
		}
		if (SpeedTreeImportData->IncludeNormalMapCheck)
		{
			UTexture* NormalTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_NORMAL]), true, false, LoadedPackages, ImportContext);
		}

		return UnrealMaterialInterface;
	}
	
	// create an unreal material asset
	auto MaterialFactory = NewObject<UMaterialFactoryNew>();
	UMaterial* UnrealMaterial = (UMaterial*)MaterialFactory->FactoryCreateNew(UMaterial::StaticClass(), Package, *FixedMaterialName, RF_Standalone|RF_Public, NULL, GWarn);
	FAssetRegistryModule::AssetCreated(UnrealMaterial);
	Package->SetDirtyFlag(true);

	// Keep track of the processed materials
	ImportContext.ImportedMaterials.Add(MaterialFullName, UnrealMaterialInterface);

	UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();

	if (!RenderState->m_bDiffuseAlphaMaskIsOpaque && !RenderState->m_bBranchesPresent && !RenderState->m_bRigidMeshesPresent)
	{
		UnrealMaterial->BlendMode = BLEND_Masked;
		UnrealMaterial->SetCastShadowAsMasked(true);
		UnrealMaterial->TwoSided = !(RenderState->m_bHorzBillboard || RenderState->m_bVertBillboard);
	}

	UMaterialExpressionClamp* BranchSeamAmount = NULL;
	if (SpeedTreeImportData->IncludeBranchSeamSmoothing && RenderState->m_bBranchesPresent && RenderState->m_eBranchSeamSmoothing != SpeedTree::EFFECT_OFF)
	{
		UMaterialExpressionTextureCoordinate* SeamTexcoordExpression = NewObject<UMaterialExpressionTextureCoordinate>(UnrealMaterial);
		SeamTexcoordExpression->CoordinateIndex = 4;
		UnrealMaterial->GetExpressionCollection().AddExpression(SeamTexcoordExpression);

		UMaterialExpressionComponentMask* ComponentMaskExpression = NewObject<UMaterialExpressionComponentMask>(UnrealMaterial);
		ComponentMaskExpression->R = 0;
		ComponentMaskExpression->G = 1;
		ComponentMaskExpression->B = 0;
		ComponentMaskExpression->A = 0;
		ComponentMaskExpression->Input.Expression = SeamTexcoordExpression;
		UnrealMaterial->GetExpressionCollection().AddExpression(ComponentMaskExpression);

		UMaterialExpressionPower* PowerExpression = NewObject<UMaterialExpressionPower>(UnrealMaterial);
		PowerExpression->Base.Expression = ComponentMaskExpression;
		PowerExpression->ConstExponent = RenderState->m_fBranchSeamWeight;
		UnrealMaterial->GetExpressionCollection().AddExpression(PowerExpression);

		BranchSeamAmount = NewObject<UMaterialExpressionClamp>(UnrealMaterial);
		BranchSeamAmount->Input.Expression = PowerExpression;
		UnrealMaterial->GetExpressionCollection().AddExpression(BranchSeamAmount);
	}

	// textures and properties
	UTexture* DiffuseTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_DIFFUSE]), false, false, LoadedPackages, ImportContext);
	if (DiffuseTexture)
	{
		// make texture sampler
		UMaterialExpressionTextureSample* TextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
		TextureExpression->Texture = DiffuseTexture;
		TextureExpression->SamplerType = SAMPLERTYPE_Color;
		UnrealMaterial->GetExpressionCollection().AddExpression(TextureExpression);

		// hook to the material diffuse/mask
		UnrealMaterialEditorOnly->BaseColor.Expression = TextureExpression;
		UnrealMaterialEditorOnly->OpacityMask.Expression = TextureExpression;
		UnrealMaterialEditorOnly->OpacityMask.Mask = UnrealMaterialEditorOnly->OpacityMask.Expression->GetOutputs()[0].Mask;
		UnrealMaterialEditorOnly->OpacityMask.MaskR = 0;
		UnrealMaterialEditorOnly->OpacityMask.MaskG = 0;
		UnrealMaterialEditorOnly->OpacityMask.MaskB = 0;
		UnrealMaterialEditorOnly->OpacityMask.MaskA = 1;

		if (BranchSeamAmount != NULL)
		{
			// perform branch seam smoothing
			UMaterialExpressionTextureCoordinate* SeamTexcoordExpression = NewObject<UMaterialExpressionTextureCoordinate>(UnrealMaterial);
			SeamTexcoordExpression->CoordinateIndex = 6;
			UnrealMaterial->GetExpressionCollection().AddExpression(SeamTexcoordExpression);

			UMaterialExpressionTextureSample* SeamTextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
			SeamTextureExpression->Texture = DiffuseTexture;
			SeamTextureExpression->SamplerType = SAMPLERTYPE_Color;
			SeamTextureExpression->Coordinates.Expression = SeamTexcoordExpression;
			UnrealMaterial->GetExpressionCollection().AddExpression(SeamTextureExpression);

			UMaterialExpressionLinearInterpolate* InterpolateExpression = NewObject<UMaterialExpressionLinearInterpolate>(UnrealMaterial);
			InterpolateExpression->A.Expression = SeamTextureExpression;
			InterpolateExpression->B.Expression = TextureExpression;
			InterpolateExpression->Alpha.Expression = BranchSeamAmount;
			UnrealMaterial->GetExpressionCollection().AddExpression(InterpolateExpression);

			UnrealMaterialEditorOnly->BaseColor.Expression = InterpolateExpression;
		}

		if (RenderState->m_bBranchesPresent && SpeedTreeImportData->IncludeDetailMapCheck)
		{
			UTexture* DetailTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_DETAIL_DIFFUSE]), false, false, LoadedPackages, ImportContext);
			if (DetailTexture)
			{
				// add/find UVSet
				UMaterialExpressionTextureCoordinate* DetailTexcoordExpression = NewObject<UMaterialExpressionTextureCoordinate>(UnrealMaterial);
				DetailTexcoordExpression->CoordinateIndex = 5;
				UnrealMaterial->GetExpressionCollection().AddExpression(DetailTexcoordExpression);
				
				// make texture sampler
				UMaterialExpressionTextureSample* DetailTextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
				DetailTextureExpression->Texture = DetailTexture;
				DetailTextureExpression->SamplerType = SAMPLERTYPE_Color;
				DetailTextureExpression->Coordinates.Expression = DetailTexcoordExpression;
				UnrealMaterial->GetExpressionCollection().AddExpression(DetailTextureExpression);

				// interpolate the detail
				UMaterialExpressionLinearInterpolate* InterpolateExpression = NewObject<UMaterialExpressionLinearInterpolate>(UnrealMaterial);
				InterpolateExpression->A.Expression = UnrealMaterialEditorOnly->BaseColor.Expression;
				InterpolateExpression->B.Expression = DetailTextureExpression;
				InterpolateExpression->Alpha.Expression = DetailTextureExpression;
				InterpolateExpression->Alpha.Mask = DetailTextureExpression->GetOutputs()[0].Mask;
				InterpolateExpression->Alpha.MaskR = 0;
				InterpolateExpression->Alpha.MaskG = 0;
				InterpolateExpression->Alpha.MaskB = 0;
				InterpolateExpression->Alpha.MaskA = 1;
				UnrealMaterial->GetExpressionCollection().AddExpression(InterpolateExpression);

				// hook final to diffuse
				UnrealMaterialEditorOnly->BaseColor.Expression = InterpolateExpression;
			}
		}
	}

	bool bMadeSpecular = false;
	if (SpeedTreeImportData->IncludeSpecularMapCheck)
	{
		UTexture* SpecularTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_SPECULAR_MASK]), false, false, LoadedPackages, ImportContext);
		if (SpecularTexture)
		{
			// make texture sampler
			UMaterialExpressionTextureSample* TextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
			TextureExpression->Texture = SpecularTexture;
			TextureExpression->SamplerType = SAMPLERTYPE_Color;
			
			UnrealMaterial->GetExpressionCollection().AddExpression(TextureExpression);
			UnrealMaterialEditorOnly->Specular.Expression = TextureExpression;
			bMadeSpecular = true;
		}
	}

	if (!bMadeSpecular)
	{
		UMaterialExpressionConstant* ZeroExpression = NewObject<UMaterialExpressionConstant>(UnrealMaterial);
		ZeroExpression->R = 0.0f;
		UnrealMaterial->GetExpressionCollection().AddExpression(ZeroExpression);
		UnrealMaterialEditorOnly->Specular.Expression = ZeroExpression;
	}

	if (SpeedTreeImportData->IncludeNormalMapCheck)
	{
		UTexture* NormalTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(RenderState->m_apTextures[SpeedTree::TL_NORMAL]), true, false, LoadedPackages, ImportContext);
		if (NormalTexture)
		{
			// make texture sampler
			UMaterialExpressionTextureSample* TextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
			TextureExpression->Texture = NormalTexture;
			TextureExpression->SamplerType = SAMPLERTYPE_Normal;
			
			UnrealMaterial->GetExpressionCollection().AddExpression(TextureExpression);
			UnrealMaterialEditorOnly->Normal.Expression = TextureExpression;

			if (BranchSeamAmount != NULL)
			{
				// perform branch seam smoothing
				UMaterialExpressionTextureCoordinate* SeamTexcoordExpression = NewObject<UMaterialExpressionTextureCoordinate>(UnrealMaterial);
				SeamTexcoordExpression->CoordinateIndex = 6;
				UnrealMaterial->GetExpressionCollection().AddExpression(SeamTexcoordExpression);

				UMaterialExpressionTextureSample* SeamTextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
				SeamTextureExpression->Texture = NormalTexture;
				SeamTextureExpression->SamplerType = SAMPLERTYPE_Normal;
				SeamTextureExpression->Coordinates.Expression = SeamTexcoordExpression;
				UnrealMaterial->GetExpressionCollection().AddExpression(SeamTextureExpression);

				UMaterialExpressionLinearInterpolate* InterpolateExpression = NewObject<UMaterialExpressionLinearInterpolate>(UnrealMaterial);
				InterpolateExpression->A.Expression = SeamTextureExpression;
				InterpolateExpression->B.Expression = TextureExpression;
				InterpolateExpression->Alpha.Expression = BranchSeamAmount;
				UnrealMaterial->GetExpressionCollection().AddExpression(InterpolateExpression);

				UnrealMaterialEditorOnly->Normal.Expression = InterpolateExpression;
			}
		}
	}

	if (SpeedTreeImportData->IncludeVertexProcessingCheck && !RenderState->m_bRigidMeshesPresent)
	{
		UMaterialExpressionSpeedTree* SpeedTreeExpression = NewObject<UMaterialExpressionSpeedTree>(UnrealMaterial);
	
		SpeedTreeExpression->LODType = (SpeedTreeImportData->IncludeSmoothLODCheck ? STLOD_Smooth : STLOD_Pop);
		SpeedTreeExpression->WindType = WindType;

		float BillboardThreshold = FMath::Clamp((float)(NumBillboards - 8) / 16.0f, 0.0f, 1.0f);
		SpeedTreeExpression->BillboardThreshold = 0.9f - BillboardThreshold * 0.8f;

		if (RenderState->m_bBranchesPresent)
			SpeedTreeExpression->GeometryType = STG_Branch;
		else if (RenderState->m_bFrondsPresent)
			SpeedTreeExpression->GeometryType = STG_Frond;
		else if (RenderState->m_bHorzBillboard || RenderState->m_bVertBillboard)
			SpeedTreeExpression->GeometryType = STG_Billboard;
		else if (RenderState->m_bLeavesPresent)
			SpeedTreeExpression->GeometryType = STG_Leaf;
		else
			SpeedTreeExpression->GeometryType = STG_FacingLeaf;

		UnrealMaterial->GetExpressionCollection().AddExpression(SpeedTreeExpression);
		UnrealMaterialEditorOnly->WorldPositionOffset.Expression = SpeedTreeExpression;
	}

	if (SpeedTreeImportData->IncludeSpeedTreeAO &&
		!(RenderState->m_bVertBillboard || RenderState->m_bHorzBillboard))
	{
		UMaterialExpressionVertexColor* VertexColor = NewObject<UMaterialExpressionVertexColor>(UnrealMaterial);
		UnrealMaterial->GetExpressionCollection().AddExpression(VertexColor);
		UnrealMaterialEditorOnly->AmbientOcclusion.Expression = VertexColor;
		UnrealMaterialEditorOnly->AmbientOcclusion.Mask = VertexColor->GetOutputs()[0].Mask;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskR = 1;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskG = 0;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskB = 0;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskA = 0;
	}

	// Unreal flips normals for two-sided materials. SpeedTrees don't need that
	if (UnrealMaterial->TwoSided)
	{
		UMaterialExpressionTwoSidedSign* TwoSidedSign = NewObject<UMaterialExpressionTwoSidedSign>(UnrealMaterial);
		UnrealMaterial->GetExpressionCollection().AddExpression(TwoSidedSign);

		auto Multiply = NewObject<UMaterialExpressionMultiply>(UnrealMaterial);
		UnrealMaterial->GetExpressionCollection().AddExpression(Multiply);
		Multiply->A.Expression = TwoSidedSign;

		if (UnrealMaterialEditorOnly->Normal.Expression == NULL)
		{
			auto VertexNormalExpression = NewObject<UMaterialExpressionConstant3Vector>(UnrealMaterial);
			UnrealMaterial->GetExpressionCollection().AddExpression(VertexNormalExpression);
			VertexNormalExpression->Constant = FLinearColor(0.0f, 0.0f, 1.0f);

			Multiply->B.Expression = VertexNormalExpression;
		}
		else
		{
			Multiply->B.Expression = UnrealMaterialEditorOnly->Normal.Expression;
		}

		UnrealMaterialEditorOnly->Normal.Expression = Multiply;
	}

	if (SpeedTreeImportData->IncludeColorAdjustment && UnrealMaterialEditorOnly->BaseColor.Expression != NULL &&
		(RenderState->m_bLeavesPresent || RenderState->m_bFacingLeavesPresent || RenderState->m_bVertBillboard || RenderState->m_bHorzBillboard))
	{
		UMaterialFunction* ColorVariationFunction = LoadObject<UMaterialFunction>(NULL, TEXT("/Engine/Functions/Engine_MaterialFunctions01/SpeedTree/SpeedTreeColorVariation.SpeedTreeColorVariation"), NULL, LOAD_None, NULL);
		if (ColorVariationFunction)
		{
			UMaterialExpressionMaterialFunctionCall* ColorVariation = NewObject<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
			UnrealMaterial->GetExpressionCollection().AddExpression(ColorVariation);

			ColorVariation->MaterialFunction = ColorVariationFunction;
			ColorVariation->UpdateFromFunctionResource( );

			ColorVariation->GetInput(0)->Expression = UnrealMaterialEditorOnly->BaseColor.Expression;
			ColorVariation->GetInput(0)->Mask = UnrealMaterialEditorOnly->BaseColor.Expression->GetOutputs()[0].Mask;
			ColorVariation->GetInput(0)->MaskR = 1;
			ColorVariation->GetInput(0)->MaskG = 1;
			ColorVariation->GetInput(0)->MaskB = 1;
			ColorVariation->GetInput(0)->MaskA = 0;

			UnrealMaterialEditorOnly->BaseColor.Expression = ColorVariation;
		}
	}
	
	LayoutMaterial(UnrealMaterial);

	// make sure that any static meshes, etc using this material will stop using the FMaterialResource of the original 
	// material, and will use the new FMaterialResource created when we make a new UMaterial in place
	FGlobalComponentReregisterContext RecreateComponents;
	
	// let the material update itself if necessary
	UnrealMaterial->PreEditChange(NULL);
	UnrealMaterial->PostEditChange();
	
	return UnrealMaterial;
}

UMaterialInterface* CreateSpeedTreeMaterial8(UObject* Parent, FString MaterialFullName, GameEngine8::CMaterial& SpeedTreeMaterial, USpeedTreeImportData* SpeedTreeImportData, ESpeedTreeWindType WindType, ESpeedTreeGeometryType GeomType, TSet<UPackage*>& LoadedPackages, bool bCrossfadeLOD, FSpeedTreeImportContext& ImportContext)
{
	// Make sure we have a parent
	if (!SpeedTreeImportData->MakeMaterialsCheck || !ensure(Parent))
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}

	if (auto* Material = ImportContext.ImportedMaterials.Find(MaterialFullName))
	{
		// The material was already imported
		return *Material;
	}

	// set where to place the materials
	FString FixedMaterialName = ObjectTools::SanitizeObjectName(MaterialFullName);
	FString NewPackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName()) + TEXT("/") + FixedMaterialName;
	UPackage* Package = UPackageTools::FindOrCreatePackageForAssetType(FName(*NewPackageName), UMaterial::StaticClass());
	check(Package);
	NewPackageName = Package->GetFullName();
	FixedMaterialName = FPaths::GetBaseFilename(NewPackageName, true);

	// does not override existing materials
	UMaterialInterface* UnrealMaterialInterface = FindObject<UMaterialInterface>(Package, *FixedMaterialName);
	if (UnrealMaterialInterface != NULL)
	{
		// Keep track of the processed materials
		ImportContext.ImportedMaterials.Add(MaterialFullName, UnrealMaterialInterface);

		// touch the textures anyway to make sure they reload if necessary
		if (SpeedTreeMaterial.Maps().Count() > 0 && SpeedTreeMaterial.Maps()[0].Used() && !SpeedTreeMaterial.Maps()[0].Path().IsEmpty())
		{
			UTexture* DiffuseTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[0].Path().Data()), false, false, LoadedPackages, ImportContext);
		}

		if (SpeedTreeMaterial.Maps().Count() > 1 && SpeedTreeMaterial.Maps()[1].Used() && !SpeedTreeMaterial.Maps()[1].Path().IsEmpty())
		{
			UTexture* NormalTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[1].Path().Data()), false, false, LoadedPackages, ImportContext);
		}

		if (SpeedTreeImportData->IncludeSubsurface && SpeedTreeMaterial.Maps().Count() > 2 && SpeedTreeMaterial.Maps()[2].Used() && !SpeedTreeMaterial.Maps()[2].Path().IsEmpty())
		{
			UTexture* SubsurfaceTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[2].Path().Data()), false, false, LoadedPackages, ImportContext);
		}

		return UnrealMaterialInterface;
	}

	// create an unreal material asset
	auto MaterialFactory = NewObject<UMaterialFactoryNew>();
	UMaterial* UnrealMaterial = (UMaterial*)MaterialFactory->FactoryCreateNew(UMaterial::StaticClass(), Package, *FixedMaterialName, RF_Standalone | RF_Public, NULL, GWarn);
	FAssetRegistryModule::AssetCreated(UnrealMaterial);
	Package->SetDirtyFlag(true);

	// Keep track of the processed materials
	ImportContext.ImportedMaterials.Add(MaterialFullName, UnrealMaterialInterface);

	UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();

	// basic setup
	UnrealMaterial->TwoSided = SpeedTreeMaterial.TwoSided();
	UnrealMaterial->BlendMode = BLEND_Masked;
	UnrealMaterial->SetCastShadowAsMasked(true);
	if (bCrossfadeLOD)
	{
		UnrealMaterial->DitheredLODTransition = true;
	}

	// vertex color for AO and blending
	UMaterialExpressionVertexColor* VertexColor = NULL;
	if (GeomType != STG_Billboard)
	{
		VertexColor = NewObject<UMaterialExpressionVertexColor>(UnrealMaterial);
		UnrealMaterial->GetExpressionCollection().AddExpression(VertexColor);
		UnrealMaterialEditorOnly->AmbientOcclusion.Expression = VertexColor;
		UnrealMaterialEditorOnly->AmbientOcclusion.Mask = VertexColor->GetOutputs()[0].Mask;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskR = 1;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskG = 0;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskB = 0;
		UnrealMaterialEditorOnly->AmbientOcclusion.MaskA = 0;
	}

	// diffuse and opacity mask
	if (SpeedTreeMaterial.Maps().Count() > 0 && SpeedTreeMaterial.Maps()[0].Used())
	{
		if (SpeedTreeMaterial.Maps()[0].Path().IsEmpty())
		{
			UMaterialExpressionConstant3Vector* ColorExpression = NewObject<UMaterialExpressionConstant3Vector>(UnrealMaterial);
			GameEngine8::Vec4 Color = SpeedTreeMaterial.Maps()[0].Color();
			ColorExpression->Constant = FLinearColor(FColor(Color.x * 255, Color.y * 255, Color.z * 255));
			UnrealMaterial->GetExpressionCollection().AddExpression(ColorExpression);
			UnrealMaterialEditorOnly->BaseColor.Expression = ColorExpression;

			if (VertexColor != NULL)
			{
				UnrealMaterialEditorOnly->OpacityMask.Expression = VertexColor;
				UnrealMaterialEditorOnly->OpacityMask.Mask = VertexColor->GetOutputs()[0].Mask;
				UnrealMaterialEditorOnly->OpacityMask.MaskR = 0;
				UnrealMaterialEditorOnly->OpacityMask.MaskG = 0;
				UnrealMaterialEditorOnly->OpacityMask.MaskB = 0;
				UnrealMaterialEditorOnly->OpacityMask.MaskA = 1;
			}
		}
		else
		{
			UTexture* DiffuseTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[0].Path().Data()), false, false, LoadedPackages, ImportContext);
			if (DiffuseTexture)
			{
				// this helps prevent mipmapping from eating away tiny leaves
				if (!FMath::IsNearlyEqual(DiffuseTexture->AdjustMinAlpha, 0.1f))
				{
					DiffuseTexture->PreEditChange(nullptr);
					DiffuseTexture->AdjustMinAlpha = 0.1f;
					DiffuseTexture->PostEditChange();
				}

				// make texture sampler
				UMaterialExpressionTextureSample* TextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
				TextureExpression->Texture = DiffuseTexture;
				TextureExpression->SamplerType = SAMPLERTYPE_Color;
				UnrealMaterial->GetExpressionCollection().AddExpression(TextureExpression);
				UnrealMaterialEditorOnly->BaseColor.Expression = TextureExpression;

				if (VertexColor == NULL)
				{
					UMaterialFunction* BillboardCrossfadeFunction = LoadObject<UMaterialFunction>(NULL, TEXT("/Engine/Functions/Engine_MaterialFunctions01/SpeedTree/SpeedTreeCrossfadeBillboard.SpeedTreeCrossfadeBillboard"), NULL, LOAD_None, NULL);
					if (BillboardCrossfadeFunction)
					{
						UMaterialExpressionMaterialFunctionCall* BillboardCrossfade = NewObject<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
						UnrealMaterial->GetExpressionCollection().AddExpression(BillboardCrossfade);
						BillboardCrossfade->MaterialFunction = BillboardCrossfadeFunction;
						BillboardCrossfade->UpdateFromFunctionResource();

						BillboardCrossfade->GetInput(0)->Expression = TextureExpression;
						BillboardCrossfade->GetInput(0)->Mask = TextureExpression->GetOutputs()[0].Mask;
						BillboardCrossfade->GetInput(0)->MaskR = 0;
						BillboardCrossfade->GetInput(0)->MaskG = 0;
						BillboardCrossfade->GetInput(0)->MaskB = 0;
						BillboardCrossfade->GetInput(0)->MaskA = 1;
						UnrealMaterialEditorOnly->OpacityMask.Expression = BillboardCrossfade;
						UnrealMaterialEditorOnly->OpacityMask.OutputIndex = 0.;

						UnrealMaterial->NumCustomizedUVs = 3;
						UnrealMaterialEditorOnly->CustomizedUVs[2].Expression = BillboardCrossfade;
						UnrealMaterialEditorOnly->CustomizedUVs[2].OutputIndex = 1;
					}
				}
				else
				{
					// make mask with blend value and opacity
					auto Multiply = NewObject<UMaterialExpressionMultiply>(UnrealMaterial);
					UnrealMaterial->GetExpressionCollection().AddExpression(Multiply);
					Multiply->B.Expression = VertexColor;
					Multiply->B.Mask = VertexColor->GetOutputs()[0].Mask;
					Multiply->B.MaskR = 0;
					Multiply->B.MaskG = 0;
					Multiply->B.MaskB = 0;
					Multiply->B.MaskA = 1;
					Multiply->A.Expression = TextureExpression;
					Multiply->A.Mask = TextureExpression->GetOutputs()[0].Mask;
					Multiply->A.MaskR = 0;
					Multiply->A.MaskG = 0;
					Multiply->A.MaskB = 0;
					Multiply->A.MaskA = 1;
					UnrealMaterialEditorOnly->OpacityMask.Expression = Multiply;
				}
			}
		}
	}

	// normal map and roughness
	if (SpeedTreeMaterial.Maps().Count() > 1 && SpeedTreeMaterial.Maps()[1].Used())
	{
		if (SpeedTreeMaterial.Maps()[1].Path().IsEmpty())
		{
			UMaterialExpressionConstant* RoughExpression = NewObject<UMaterialExpressionConstant>(UnrealMaterial);
			RoughExpression->R = SpeedTreeMaterial.Maps()[1].Color().w;
			UnrealMaterial->GetExpressionCollection().AddExpression(RoughExpression);
			UnrealMaterialEditorOnly->Roughness.Expression = RoughExpression;
		}
		else
		{
			// default SpeedTree v8 texture packing uses one texture for normal and roughness. BC5 doesn't support alpha, so compress with default settings, but disable sRGB. Then bias the normal ourselves

			UTexture* NormalTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[1].Path().Data()), false, false, LoadedPackages, ImportContext);
			if (NormalTexture)
			{
				if (NormalTexture->SRGB != false)
				{
					NormalTexture->PreEditChange(nullptr);
					NormalTexture->SRGB = false;
					NormalTexture->PostEditChange();
				}

				// make texture sampler
				UMaterialExpressionTextureSample* TextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
				TextureExpression->Texture = NormalTexture;
				TextureExpression->SamplerType = SAMPLERTYPE_LinearColor;
				UnrealMaterial->GetExpressionCollection().AddExpression(TextureExpression);

				UMaterialExpressionConstantBiasScale* BiasScale = NewObject<UMaterialExpressionConstantBiasScale>(UnrealMaterial);
				UnrealMaterial->GetExpressionCollection().AddExpression(BiasScale);
				BiasScale->Bias = -0.5f;
				BiasScale->Scale = 2.0f;
				BiasScale->Input.Expression = TextureExpression;

				if (GeomType == STG_Billboard)
				{
					UMaterialFunction* BillboardNormalFunction = LoadObject<UMaterialFunction>(NULL, TEXT("/Engine/Functions/Engine_MaterialFunctions01/SpeedTree/SpeedTreeBillboardNormals.SpeedTreeBillboardNormals"), NULL, LOAD_None, NULL);
					if (BillboardNormalFunction)
					{
						UMaterialExpressionMaterialFunctionCall* BillboardNormals = NewObject<UMaterialExpressionMaterialFunctionCall>(UnrealMaterial);
						UnrealMaterial->GetExpressionCollection().AddExpression(BillboardNormals);
						BillboardNormals->MaterialFunction = BillboardNormalFunction;
						BillboardNormals->UpdateFromFunctionResource();

						BillboardNormals->GetInput(0)->Expression = BiasScale;
						BillboardNormals->GetInput(0)->Mask = BiasScale->GetOutputs()[0].Mask;
						BillboardNormals->GetInput(0)->MaskR = 1;
						BillboardNormals->GetInput(0)->MaskG = 1;
						BillboardNormals->GetInput(0)->MaskB = 1;
						BillboardNormals->GetInput(0)->MaskA = 0;

						UnrealMaterialEditorOnly->Normal.Expression = BillboardNormals;
						UnrealMaterial->bTangentSpaceNormal = false;
					}
				}

				if (UnrealMaterialEditorOnly->Normal.Expression == NULL)
				{
					UnrealMaterialEditorOnly->Normal.Expression = BiasScale;
				}

				// roughness
				UnrealMaterialEditorOnly->Roughness.Expression = TextureExpression;
				UnrealMaterialEditorOnly->Roughness.Mask = TextureExpression->GetOutputs()[0].Mask;
				UnrealMaterialEditorOnly->Roughness.MaskR = 0;
				UnrealMaterialEditorOnly->Roughness.MaskG = 0;
				UnrealMaterialEditorOnly->Roughness.MaskB = 0;
				UnrealMaterialEditorOnly->Roughness.MaskA = 1;
			}
		}
	}

	// subsurface map
	if (SpeedTreeImportData->IncludeSubsurface && SpeedTreeMaterial.Maps().Count() > 2 && SpeedTreeMaterial.Maps()[2].Used())
	{
		if (SpeedTreeMaterial.Maps()[2].Path().IsEmpty())
		{
			UMaterialExpressionConstant3Vector* ColorExpression = NewObject<UMaterialExpressionConstant3Vector>(UnrealMaterial);
			GameEngine8::Vec4 Color = SpeedTreeMaterial.Maps()[2].Color();
			ColorExpression->Constant = FLinearColor(FColor(Color.x * 255, Color.y * 255, Color.z * 255));
			UnrealMaterial->GetExpressionCollection().AddExpression(ColorExpression);
			UnrealMaterialEditorOnly->SubsurfaceColor.Expression = ColorExpression;
		}
		else
		{
			UTexture* SubsurfaceTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[2].Path().Data()), false, false, LoadedPackages, ImportContext);
			if (SubsurfaceTexture)
			{
				// make texture sampler
				UMaterialExpressionTextureSample* TextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
				TextureExpression->Texture = SubsurfaceTexture;
				TextureExpression->SamplerType = SAMPLERTYPE_Color;
				UnrealMaterial->GetExpressionCollection().AddExpression(TextureExpression);
				UnrealMaterialEditorOnly->SubsurfaceColor.Expression = TextureExpression;

				UnrealMaterial->SetShadingModel(MSM_TwoSidedFoliage);
			}
		}
	}

	// SpeedTree node
	if (SpeedTreeImportData->IncludeVertexProcessingCheck)
	{
		UMaterialExpressionSpeedTree* SpeedTreeExpression = NewObject<UMaterialExpressionSpeedTree>(UnrealMaterial);
		SpeedTreeExpression->LODType = (bCrossfadeLOD ? STLOD_Pop : STLOD_Smooth);
		SpeedTreeExpression->WindType = WindType;
		SpeedTreeExpression->BillboardThreshold = 1.0f; // billboards use crossfade technique now in v8
		SpeedTreeExpression->GeometryType = GeomType;
		UnrealMaterial->GetExpressionCollection().AddExpression(SpeedTreeExpression);
		UnrealMaterialEditorOnly->WorldPositionOffset.Expression = SpeedTreeExpression;
	}

	LayoutMaterial(UnrealMaterial, true);

	// make sure that any static meshes, etc using this material will stop using the FMaterialResource of the original 
	// material, and will use the new FMaterialResource created when we make a new UMaterial in place
	FGlobalComponentReregisterContext RecreateComponents;

	// let the material update itself if necessary
	UnrealMaterial->PreEditChange(NULL);
	UnrealMaterial->PostEditChange();

	return UnrealMaterial;
}


UMaterialInterface* CreateSpeedTreeMaterial9(UObject* Parent, FString MaterialFullName, GameEngine9::CMaterial& SpeedTreeMaterial, UMaterialInterface* TemplateMaterial, GameEngine9::CWindConfigSDK Wind, bool bHandleCameraFacing, int32 BillboardCount, TSet<UPackage*>& LoadedPackages, FSpeedTreeImportContext& ImportContext)
{
	// Make sure we have a parent
	if ((TemplateMaterial == NULL) || !ensure(Parent))
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}

	if (auto* Material = ImportContext.ImportedMaterials.Find(MaterialFullName))
	{
		// The material was already imported
		return *Material;
	}

	// set where to place the materials
	FString FixedMaterialName = ObjectTools::SanitizeObjectName(MaterialFullName);
	FString BasePackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName()) / FixedMaterialName;
	BasePackageName = UPackageTools::SanitizePackageName(BasePackageName);

	UMaterialInterface* ExistingMaterial = nullptr;
	UPackage* Package = nullptr;
	{
		FString ObjectPath = BasePackageName + TEXT(".") + FixedMaterialName;
		ExistingMaterial = LoadObject<UMaterialInterface>(nullptr, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn);
	}

	if (ExistingMaterial)
	{
		Package = ExistingMaterial->GetOutermost();
	}
	else
	{
		const FString Suffix(TEXT(""));

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FString FinalPackageName;
		AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, Suffix, FinalPackageName, FixedMaterialName);

		Package = CreatePackage(*FinalPackageName);
	}

	// does not override existing materials
	if (ExistingMaterial != nullptr)
	{
		// Keep track of the processed materials
		ImportContext.ImportedMaterials.Add(MaterialFullName, ExistingMaterial);

		// touch the textures anyway to make sure they reload if necessary
		if (SpeedTreeMaterial.Maps().Count() > 0 && SpeedTreeMaterial.Maps()[0].Used() && !SpeedTreeMaterial.Maps()[0].Path().IsEmpty())
		{
			UTexture* DiffuseTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[0].Path().Data()), false, false, LoadedPackages, ImportContext);
		}

		if (SpeedTreeMaterial.Maps().Count() > 1 && SpeedTreeMaterial.Maps()[1].Used() && !SpeedTreeMaterial.Maps()[1].Path().IsEmpty())
		{
			UTexture* NormalTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[1].Path().Data()), false, false, LoadedPackages, ImportContext);
		}

		if (SpeedTreeMaterial.Maps().Count() > 2 && SpeedTreeMaterial.Maps()[2].Used() && !SpeedTreeMaterial.Maps()[2].Path().IsEmpty())
		{
			UTexture* SubsurfaceTexture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[2].Path().Data()), false, false, LoadedPackages, ImportContext);
		}

		return ExistingMaterial;
	}

	// create a material instance
	auto MaterialFactory = NewObject<UMaterialInstanceConstantFactoryNew>();
	UMaterialInstanceConstant* UnrealMaterialInstance = (UMaterialInstanceConstant*)MaterialFactory->FactoryCreateNew(UMaterialInstanceConstant::StaticClass(), Package, *FixedMaterialName, RF_Standalone | RF_Public, NULL, GWarn);
	FAssetRegistryModule::AssetCreated(UnrealMaterialInstance);
	UnrealMaterialInstance->SetParentEditorOnly(TemplateMaterial);
	UnrealMaterialInstance->ClearParameterValuesEditorOnly();
	Package->SetDirtyFlag(true);
	UnrealMaterialInstance->PreEditChange(NULL);
	UnrealMaterialInstance->PostEditChange();

	// Keep track of the processed materials
	ImportContext.ImportedMaterials.Add(MaterialFullName, UnrealMaterialInstance);

	// base material property overrides
	if (UnrealMaterialInstance->IsTwoSided() != SpeedTreeMaterial.TwoSided())
	{
		UnrealMaterialInstance->BasePropertyOverrides.bOverride_TwoSided = true;
		UnrealMaterialInstance->BasePropertyOverrides.TwoSided = SpeedTreeMaterial.TwoSided();
	}
	EMaterialShadingModel WantedShadingModel = ((SpeedTreeMaterial.TwoSided() || SpeedTreeMaterial.Billboard()) ? MSM_TwoSidedFoliage : MSM_DefaultLit);
	if (!UnrealMaterialInstance->ShadingModels.HasShadingModel(WantedShadingModel))
	{
		UnrealMaterialInstance->BasePropertyOverrides.bOverride_ShadingModel = true;
		UnrealMaterialInstance->BasePropertyOverrides.ShadingModel = WantedShadingModel;
	}

	// material static setup
	FStaticParameterSet StaticParameters;
	UnrealMaterialInstance->GetStaticParameterValues(StaticParameters);
	for (FStaticSwitchParameter& SwitchParameter : StaticParameters.StaticSwitchParameters)
	{
		if (SwitchParameter.ParameterInfo.Name == FName(TEXT("SharedEnable")))
		{
			SwitchParameter.Value = Wind.DoShared();
			SwitchParameter.bOverride = true;
		}
		else if (SwitchParameter.ParameterInfo.Name == FName(TEXT("Branch1Enable")))
		{
			SwitchParameter.Value = Wind.DoBranch1();
			SwitchParameter.bOverride = true;
		}
		else if (SwitchParameter.ParameterInfo.Name == FName(TEXT("Branch2Enable")))
		{
			SwitchParameter.Value = Wind.DoBranch2();
			SwitchParameter.bOverride = true;
		}
		else if (SwitchParameter.ParameterInfo.Name == FName(TEXT("RippleEnable")))
		{
			SwitchParameter.Value = Wind.DoRipple();
			SwitchParameter.bOverride = true;
		}
		else if (SwitchParameter.ParameterInfo.Name == FName(TEXT("RippleShimmerEnable")))
		{
			SwitchParameter.Value = Wind.DoShimmer();
			SwitchParameter.bOverride = true;
		}
		else if (SwitchParameter.ParameterInfo.Name == FName(TEXT("FlipBacksideNormals")))
		{
			SwitchParameter.Value = SpeedTreeMaterial.FlipNormalsOnBackside();
			SwitchParameter.bOverride = true;
		}
		else if (SwitchParameter.ParameterInfo.Name == FName(TEXT("HandleCameraFacing")))
		{
			SwitchParameter.Value = bHandleCameraFacing;
			SwitchParameter.bOverride = true;
		}
		else if (SwitchParameter.ParameterInfo.Name == FName(TEXT("HasBranch2Data_Internal")))
		{
			SwitchParameter.Value = Wind.DoBranch2();
			SwitchParameter.bOverride = true;
		}
	}
	UnrealMaterialInstance->UpdateStaticPermutation(StaticParameters);

	// set wind values
	int32 WindCurveIndex = FMath::Clamp<int32>(static_cast<int32>(Wind.Shared().Bend().Count() * Wind.Common().CurrentStrength()), 0, static_cast<int32>(Wind.Shared().Bend().Count()) - 1);
	#define SET_ST_PARAM(a) UnrealMaterialInstance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo(#a), Wind.a());
	#define SET_ST_SUB_PARAM(a, b) UnrealMaterialInstance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo(#a#b), Wind.a().b());
	#define SET_ST_INDEXED_PARAM(a, b) UnrealMaterialInstance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo(#a#b), Wind.a().b()[WindCurveIndex]);

	SET_ST_INDEXED_PARAM(Shared, Bend);
	SET_ST_INDEXED_PARAM(Shared, Oscillation);
	SET_ST_INDEXED_PARAM(Shared, Speed);
	SET_ST_INDEXED_PARAM(Shared, Turbulence);
	SET_ST_INDEXED_PARAM(Shared, Flexibility);
	//SET_ST_SUB_PARAM(Shared, Independence);
	SET_ST_PARAM(SharedStartHeight);
	UnrealMaterialInstance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo("SharedIndependence"), 0.01f);

	SET_ST_INDEXED_PARAM(Branch1, Bend);
	SET_ST_INDEXED_PARAM(Branch1, Oscillation);
	SET_ST_INDEXED_PARAM(Branch1, Speed);
	SET_ST_INDEXED_PARAM(Branch1, Turbulence);
	SET_ST_INDEXED_PARAM(Branch1, Flexibility);
	SET_ST_SUB_PARAM(Branch1, Independence);
	SET_ST_PARAM(Branch1StretchLimit);

	SET_ST_INDEXED_PARAM(Branch2, Bend);
	SET_ST_INDEXED_PARAM(Branch2, Oscillation);
	SET_ST_INDEXED_PARAM(Branch2, Speed);
	SET_ST_INDEXED_PARAM(Branch2, Turbulence);
	SET_ST_INDEXED_PARAM(Branch2, Flexibility);
	SET_ST_SUB_PARAM(Branch2, Independence);
	SET_ST_PARAM(Branch2StretchLimit);

	SET_ST_INDEXED_PARAM(Ripple, Planar);
	SET_ST_INDEXED_PARAM(Ripple, Directional);
	SET_ST_INDEXED_PARAM(Ripple, Speed);
	SET_ST_INDEXED_PARAM(Ripple, Flexibility);
	SET_ST_SUB_PARAM(Ripple, Shimmer);
	SET_ST_SUB_PARAM(Ripple, Independence);

	UnrealMaterialInstance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo("BillboardCount"), BillboardCount);

	// set textures
	if (SpeedTreeMaterial.Maps().Count() > 0 && SpeedTreeMaterial.Maps()[0].Used() && !SpeedTreeMaterial.Maps()[0].Path().IsEmpty())
	{
		UTexture* Texture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[0].Path().Data()), false, false, LoadedPackages, ImportContext);
		if (Texture)
		{
			// this helps prevent mipmapping from eating away tiny leaves
			if (!FMath::IsNearlyEqual(Texture->AdjustMinAlpha, 0.05f))
			{
				Texture->PreEditChange(nullptr);
				Texture->AdjustMinAlpha = 0.05f;
				Texture->PostEditChange();
			}

			UnrealMaterialInstance->SetTextureParameterValueEditorOnly(FMaterialParameterInfo("ColorOpacity"), Texture);
		}
	}

	if (SpeedTreeMaterial.Maps().Count() > 1 && SpeedTreeMaterial.Maps()[1].Used() && !SpeedTreeMaterial.Maps()[1].Path().IsEmpty())
	{
		UTexture* Texture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[1].Path().Data()), false, false, LoadedPackages, ImportContext);
		if (Texture)
		{
			if (Texture->SRGB != false)
			{
				Texture->PreEditChange(nullptr);
				Texture->SRGB = false;
				Texture->PostEditChange();
			}

			UnrealMaterialInstance->SetTextureParameterValueEditorOnly(FMaterialParameterInfo("NormalRoughness"), Texture);
		}
	}

	if (SpeedTreeMaterial.Maps().Count() > 2 && SpeedTreeMaterial.Maps()[2].Used() && !SpeedTreeMaterial.Maps()[2].Path().IsEmpty())
	{
		UTexture* Texture = CreateSpeedTreeMaterialTexture(Parent, ANSI_TO_TCHAR(SpeedTreeMaterial.Maps()[2].Path().Data()), false, false, LoadedPackages, ImportContext);
		if (Texture)
		{
			UnrealMaterialInstance->SetTextureParameterValueEditorOnly(FMaterialParameterInfo("Subsurface"), Texture);
		}
	}

	// make sure that any static meshes, etc using this material will stop using the FMaterialResource of the original
	// material, and will use the new FMaterialResource created when we make a new UMaterial in place
	FGlobalComponentReregisterContext RecreateComponents;

	// let the material update itself if necessary
	UnrealMaterialInstance->PreEditChange(NULL);
	UnrealMaterialInstance->PostEditChange();

	return UnrealMaterialInstance;
}

static void CopySpeedTreeWind7(const SpeedTree::CWind* Wind, TSharedPtr<FSpeedTreeWind> SpeedTreeWind)
{
	SpeedTree::CWind::SParams OrigParams = Wind->GetParams();
	FSpeedTreeWind::SParams NewParams;

	#define COPY_PARAM(name) NewParams.name = OrigParams.name;
	#define COPY_CURVE(name) for (int32 CurveIndex = 0; CurveIndex < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++CurveIndex) { NewParams.name[CurveIndex] = OrigParams.name[CurveIndex]; }

	COPY_PARAM(m_fStrengthResponse);
	COPY_PARAM(m_fDirectionResponse);

	COPY_PARAM(m_fAnchorOffset);
	COPY_PARAM(m_fAnchorDistanceScale);

	for (int32 OscIndex = 0; OscIndex < FSpeedTreeWind::NUM_OSC_COMPONENTS; ++OscIndex)
	{
		COPY_CURVE(m_afFrequencies[OscIndex]);
	}

	COPY_PARAM(m_fGlobalHeight);
	COPY_PARAM(m_fGlobalHeightExponent);
	COPY_CURVE(m_afGlobalDistance);
	COPY_CURVE(m_afGlobalDirectionAdherence);

	for (int32 BranchIndex = 0; BranchIndex < FSpeedTreeWind::NUM_BRANCH_LEVELS; ++BranchIndex)
	{
		COPY_CURVE(m_asBranch[BranchIndex].m_afDistance);
		COPY_CURVE(m_asBranch[BranchIndex].m_afDirectionAdherence);
		COPY_CURVE(m_asBranch[BranchIndex].m_afWhip);
		COPY_PARAM(m_asBranch[BranchIndex].m_fTurbulence);
		COPY_PARAM(m_asBranch[BranchIndex].m_fTwitch);
		COPY_PARAM(m_asBranch[BranchIndex].m_fTwitchFreqScale);
	}

	for (int32 LeafIndex = 0; LeafIndex < FSpeedTreeWind::NUM_LEAF_GROUPS; ++LeafIndex)
	{
		COPY_CURVE(m_asLeaf[LeafIndex].m_afRippleDistance);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTumbleFlip);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTumbleTwist);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTumbleDirectionAdherence);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTwitchThrow);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fTwitchSharpness);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fRollMaxScale);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fRollMinScale);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fRollSpeed);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fRollSeparation);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fLeewardScalar);
	}

	COPY_CURVE(m_afFrondRippleDistance);
	COPY_PARAM(m_fFrondRippleTile);
	COPY_PARAM(m_fFrondRippleLightingScalar);

	COPY_PARAM(m_fGustFrequency);
	COPY_PARAM(m_fGustStrengthMin);
	COPY_PARAM(m_fGustStrengthMax);
	COPY_PARAM(m_fGustDurationMin);
	COPY_PARAM(m_fGustDurationMax);
	COPY_PARAM(m_fGustRiseScalar);
	COPY_PARAM(m_fGustFallScalar);

	SpeedTreeWind->SetParams(NewParams);

	for (int32 OptionIndex = 0; OptionIndex < FSpeedTreeWind::NUM_WIND_OPTIONS; ++OptionIndex)
	{
		SpeedTreeWind->SetOption((FSpeedTreeWind::EOptions)OptionIndex, Wind->IsOptionEnabled((SpeedTree::CWind::EOptions)OptionIndex));
	}

	const SpeedTree::st_float32* BranchAnchor = Wind->GetBranchAnchor();
	SpeedTreeWind->SetTreeValues(FVector(BranchAnchor[0], BranchAnchor[1], BranchAnchor[2]), Wind->GetMaxBranchLength());

	SpeedTreeWind->SetNeedsReload(true);

	#undef COPY_PARAM
	#undef COPY_CURVE
}

static void CopySpeedTreeWind8(const GameEngine8::SWindConfig* Wind, TSharedPtr<FSpeedTreeWind> SpeedTreeWind)
{
	FSpeedTreeWind::SParams NewParams;

	#define COPY_PARAM(name) NewParams.name = Wind->name;
	#define COPY_CURVE(name) for (int32 CurveIndex = 0; CurveIndex < FSpeedTreeWind::NUM_WIND_POINTS_IN_CURVE; ++CurveIndex) { NewParams.name[CurveIndex] = Wind->name[CurveIndex]; }

	COPY_PARAM(m_fStrengthResponse);
	COPY_PARAM(m_fDirectionResponse);

	COPY_PARAM(m_fAnchorOffset);
	COPY_PARAM(m_fAnchorDistanceScale);

	for (int32 OscIndex = 0; OscIndex < FSpeedTreeWind::NUM_OSC_COMPONENTS; ++OscIndex)
	{
		COPY_CURVE(m_afFrequencies[OscIndex]);
	}

	COPY_PARAM(m_fGlobalHeight);
	COPY_PARAM(m_fGlobalHeightExponent);
	COPY_CURVE(m_afGlobalDistance);
	COPY_CURVE(m_afGlobalDirectionAdherence);

	for (int32 BranchIndex = 0; BranchIndex < FSpeedTreeWind::NUM_BRANCH_LEVELS; ++BranchIndex)
	{
		COPY_CURVE(m_asBranch[BranchIndex].m_afDistance);
		COPY_CURVE(m_asBranch[BranchIndex].m_afDirectionAdherence);
		COPY_CURVE(m_asBranch[BranchIndex].m_afWhip);
		COPY_PARAM(m_asBranch[BranchIndex].m_fTurbulence);
		COPY_PARAM(m_asBranch[BranchIndex].m_fTwitch);
		COPY_PARAM(m_asBranch[BranchIndex].m_fTwitchFreqScale);
	}

	for (int32 LeafIndex = 0; LeafIndex < FSpeedTreeWind::NUM_LEAF_GROUPS; ++LeafIndex)
	{
		COPY_CURVE(m_asLeaf[LeafIndex].m_afRippleDistance);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTumbleFlip);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTumbleTwist);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTumbleDirectionAdherence);
		COPY_CURVE(m_asLeaf[LeafIndex].m_afTwitchThrow);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fTwitchSharpness);
		COPY_PARAM(m_asLeaf[LeafIndex].m_fLeewardScalar);
	}

	COPY_CURVE(m_afFrondRippleDistance);
	COPY_PARAM(m_fFrondRippleTile);
	COPY_PARAM(m_fFrondRippleLightingScalar);

	COPY_PARAM(m_fGustFrequency);
	COPY_PARAM(m_fGustStrengthMin);
	COPY_PARAM(m_fGustStrengthMax);
	COPY_PARAM(m_fGustDurationMin);
	COPY_PARAM(m_fGustDurationMax);
	COPY_PARAM(m_fGustRiseScalar);
	COPY_PARAM(m_fGustFallScalar);

	SpeedTreeWind->SetParams(NewParams);

	for (int32 OptionIndex = 0; OptionIndex < FSpeedTreeWind::NUM_WIND_OPTIONS; ++OptionIndex)
	{
		SpeedTreeWind->SetOption((FSpeedTreeWind::EOptions)OptionIndex, Wind->m_abOptions[OptionIndex] != 0);
	}

	SpeedTreeWind->SetTreeValues(FVector(Wind->m_vFrondStyleBranchAnchor[0], Wind->m_vFrondStyleBranchAnchor[1], Wind->m_vFrondStyleBranchAnchor[2]), Wind->m_fMaxBranchLength);

	SpeedTreeWind->SetNeedsReload(true);

	#undef COPY_PARAM
	#undef COPY_CURVE
}

static void MakeBodyFromCollisionObjects7(UStaticMesh* StaticMesh, const SpeedTree::SCollisionObject* CollisionObjects, int32 NumCollisionObjects)
{
	StaticMesh->CreateBodySetup();
	FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

	for (int32 CollisionObjectIndex = 0; CollisionObjectIndex < NumCollisionObjects; ++CollisionObjectIndex)
	{
		const SpeedTree::SCollisionObject& CollisionObject = CollisionObjects[CollisionObjectIndex];
		const FVector Pos1(-CollisionObject.m_vCenter1.x, CollisionObject.m_vCenter1.y, CollisionObject.m_vCenter1.z);
		const FVector Pos2(-CollisionObject.m_vCenter2.x, CollisionObject.m_vCenter2.y, CollisionObject.m_vCenter2.z);

		if (Pos1 == Pos2)
		{
			// sphere object
			FKSphereElem SphereElem;
			SphereElem.Radius = CollisionObject.m_fRadius;
			SphereElem.Center = Pos1;
			AggGeo.SphereElems.Add(SphereElem);
		}
		else
		{
			// capsule/sphyll object
			FKSphylElem SphylElem;
			SphylElem.Radius = CollisionObject.m_fRadius;
			FVector UpDir = Pos2 - Pos1;
			SphylElem.Length = UpDir.Size();
			if (SphylElem.Length != 0.0f)
				UpDir /= SphylElem.Length;			
			SphylElem.SetTransform( FTransform( FQuat::FindBetween( FVector( 0.0f, 0.0f, 1.0f ), UpDir ), ( Pos1 + Pos2 ) * 0.5f ) );
			AggGeo.SphylElems.Add(SphylElem);
		}
	}

	StaticMesh->GetBodySetup()->ClearPhysicsMeshes();
	StaticMesh->GetBodySetup()->InvalidatePhysicsData();
	RefreshCollisionChange(*StaticMesh);
}

static void MakeBodyFromCollisionObjects8(UStaticMesh* StaticMesh, SpeedTreeDataBuffer::CTableArray<GameEngine8::CCollisionObject> aObjects)
{
	if (aObjects.Count() > 0)
	{
		StaticMesh->CreateBodySetup();
		StaticMesh->GetBodySetup()->ClearPhysicsMeshes();
		StaticMesh->GetBodySetup()->RemoveSimpleCollision();
		StaticMesh->GetBodySetup()->CollisionTraceFlag = CTF_UseSimpleAsComplex;

		FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

		for (uint32 CollisionObjectIndex = 0; CollisionObjectIndex < aObjects.Count(); ++CollisionObjectIndex)
		{
			GameEngine8::CCollisionObject CollisionObject = aObjects[CollisionObjectIndex];
			const FVector Pos1(CollisionObject.Position().x, CollisionObject.Position().y, CollisionObject.Position().z);
			const FVector Pos2(CollisionObject.Position2().x, CollisionObject.Position2().y, CollisionObject.Position2().z);

			if (Pos1 == Pos2)
			{
				// sphere object
				FKSphereElem SphereElem;
				SphereElem.Radius = CollisionObject.Radius();
				SphereElem.Center = Pos1;
				AggGeo.SphereElems.Add(SphereElem);
			}
			else
			{
				// capsule/sphyll object
				FKSphylElem SphylElem;
				SphylElem.Radius = CollisionObject.Radius();
				FVector UpDir = Pos2 - Pos1;
				SphylElem.Length = UpDir.Size();
				if (SphylElem.Length != 0.0f)
					UpDir /= SphylElem.Length;
				SphylElem.SetTransform(FTransform(FQuat::FindBetween(FVector(0.0f, 0.0f, 1.0f), UpDir), (Pos1 + Pos2) * 0.5f));
				AggGeo.SphylElems.Add(SphylElem);
			}
		}
		
		StaticMesh->GetBodySetup()->InvalidatePhysicsData();
		RefreshCollisionChange(*StaticMesh);
	}
}

static void MakeBodyFromCollisionObjects9(UStaticMesh* StaticMesh, SpeedTreeDataBuffer::CTableArray<GameEngine9::CCollisionObject> aObjects)
{
	if (aObjects.Count() > 0)
	{
		StaticMesh->CreateBodySetup();
		StaticMesh->GetBodySetup()->ClearPhysicsMeshes();
		StaticMesh->GetBodySetup()->RemoveSimpleCollision();
		StaticMesh->GetBodySetup()->CollisionTraceFlag = CTF_UseSimpleAsComplex;

		FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

		for (uint32 CollisionObjectIndex = 0; CollisionObjectIndex < aObjects.Count(); ++CollisionObjectIndex)
		{
			GameEngine9::CCollisionObject CollisionObject = aObjects[CollisionObjectIndex];
			const FVector Pos1(CollisionObject.Position().x, CollisionObject.Position().y, CollisionObject.Position().z);
			const FVector Pos2(CollisionObject.Position2().x, CollisionObject.Position2().y, CollisionObject.Position2().z);

			if (Pos1 == Pos2)
			{
				// sphere object
				FKSphereElem SphereElem;
				SphereElem.Radius = CollisionObject.Radius();
				SphereElem.Center = Pos1;
				AggGeo.SphereElems.Add(SphereElem);
			}
			else
			{
				// capsule/sphyll object
				FKSphylElem SphylElem;
				SphylElem.Radius = CollisionObject.Radius();
				FVector UpDir = Pos2 - Pos1;
				SphylElem.Length = UpDir.Size();
				if (SphylElem.Length != 0.0f)
					UpDir /= SphylElem.Length;
				SphylElem.SetTransform(FTransform(FQuat::FindBetween(FVector(0.0f, 0.0f, 1.0f), UpDir), (Pos1 + Pos2) * 0.5f));
				AggGeo.SphylElems.Add(SphylElem);
			}
		}

		StaticMesh->GetBodySetup()->InvalidatePhysicsData();
		RefreshCollisionChange(*StaticMesh);
	}
}


FVertexInstanceID ProcessTriangleCorner(
	SpeedTree::CCore& SpeedTree,
	const int32 TriangleIndex,
	const int32 Corner,
	const SpeedTree::SDrawCall* DrawCall,
	const SpeedTree::st_uint32* Indices32,
	const SpeedTree::st_uint16* Indices16,
	FMeshDescription& MeshDescription,
	const int32 IndexOffset,
	const int32 NumUVs,
	const SpeedTree::SRenderState* RenderState,
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals,
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents,
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns,
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors,
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs)
{
	//Speedtree uses 7 or 8 UVs to store is data
	check(VertexInstanceUVs.GetNumChannels() >= 7);

	SpeedTree::st_float32 Data[ 4 ];

	// flip the triangle winding
	int32 Index = TriangleIndex * 3 + Corner;

	int32 VertexIndex = DrawCall->m_b32BitIndices ? Indices32[ Index ] : Indices16[ Index ];
	FVertexID VertexID(IndexOffset + VertexIndex);
	const FVertexInstanceID& VertexInstanceID = MeshDescription.CreateVertexInstance(VertexID);

	// tangents
	DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_NORMAL, VertexIndex, Data );
	FVector Normal( -Data[ 0 ], Data[ 1 ], Data[ 2 ] );
	DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_TANGENT, VertexIndex, Data );
	FVector Tangent( -Data[ 0 ], Data[ 1 ], Data[ 2 ] );
	VertexInstanceTangents[VertexInstanceID] = (FVector3f)Tangent;
	VertexInstanceNormals[VertexInstanceID] = (FVector3f)Normal;
	VertexInstanceBinormalSigns[VertexInstanceID] = GetBasisDeterminantSign(Tangent.GetSafeNormal(), (Normal ^ Tangent).GetSafeNormal(), Normal.GetSafeNormal());

	// ao
	DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_AMBIENT_OCCLUSION, VertexIndex, Data );
	uint8 AO = Data[ 0 ] * 255.0f;
	VertexInstanceColors[VertexInstanceID] = FLinearColor(FColor(AO, AO, AO, 255));

	// keep texcoords padded to align indices
	for( int32 PadIndex = 0; PadIndex < NumUVs; ++PadIndex )
	{
		VertexInstanceUVs.Set(VertexInstanceID, PadIndex, FVector2f::ZeroVector);
	}

	// All texcoords are packed into 4 float4 vertex attributes
	// Data is as follows								

	//		Branches			Fronds				Leaves				Billboards
	//
	// 0	Diffuse				Diffuse				Diffuse				Diffuse			
	// 1	Lightmap UV			Lightmap UV			Lightmap UV			Lightmap UV	(same as diffuse)		
	// 2	Branch Wind XY		Branch Wind XY		Branch Wind XY			
	// 3	LOD XY				LOD XY				LOD XY				
	// 4	LOD Z, Seam Amount	LOD Z, 0			LOD Z, Anchor X		
	// 5	Detail UV			Frond Wind XY		Anchor YZ	
	// 6	Seam UV				Frond Wind Z, 0		Leaf Wind XY
	// 7	0					0					Leaf Wind Z, Leaf Group


	// diffuse
	DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_DIFFUSE_TEXCOORDS, VertexIndex, Data );
	VertexInstanceUVs.Set( VertexInstanceID, 0, FVector2f( Data[ 0 ], Data[ 1 ] ) );

	// lightmap
	DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_LIGHTMAP_TEXCOORDS, VertexIndex, Data );
	VertexInstanceUVs.Set( VertexInstanceID, 1, FVector2f( Data[ 0 ], Data[ 1 ] ) );

	// branch wind
	DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_WIND_BRANCH_DATA, VertexIndex, Data );
	VertexInstanceUVs.Set( VertexInstanceID, 2, FVector2f( Data[ 0 ], Data[ 1 ] ) );

	// lod
	if( RenderState->m_bFacingLeavesPresent )
	{
		DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_LEAF_CARD_LOD_SCALAR, VertexIndex, Data );
		VertexInstanceUVs.Set( VertexInstanceID, 3, FVector2f( Data[ 0 ], 0.0f ) );
		VertexInstanceUVs.Set( VertexInstanceID, 4, FVector2f( 0.0f, 0.0f ) );
	}
	else
	{
		DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_LOD_POSITION, VertexIndex, Data );
		VertexInstanceUVs.Set( VertexInstanceID, 3, FVector2f( -Data[ 0 ], Data[ 1 ] ) );
		VertexInstanceUVs.Set( VertexInstanceID, 4, FVector2f( Data[ 2 ], 0.0f ) );
	}

	// other
	if( RenderState->m_bBranchesPresent )
	{
		// detail
		DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_DETAIL_TEXCOORDS, VertexIndex, Data );
		VertexInstanceUVs.Set( VertexInstanceID, 5, FVector2f( Data[ 0 ], Data[ 1 ] ) );

		// branch seam
		DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_BRANCH_SEAM_DIFFUSE, VertexIndex, Data );
		VertexInstanceUVs.Set( VertexInstanceID, 6, FVector2f( Data[ 0 ], Data[ 1 ] ) );
		VertexInstanceUVs.Set( VertexInstanceID, 4, FVector2f( VertexInstanceUVs.Get( VertexInstanceID, 4 ).X, Data[ 2 ] ) );
	}
	else if( RenderState->m_bFrondsPresent )
	{
		// frond wind
		DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_WIND_EXTRA_DATA, VertexIndex, Data );
		VertexInstanceUVs.Set( VertexInstanceID, 5, FVector2f( Data[ 0 ], Data[ 1 ] ) );
		VertexInstanceUVs.Set( VertexInstanceID, 6, FVector2f( Data[ 2 ], 0.0f ) );
	}
	else if( RenderState->m_bLeavesPresent || RenderState->m_bFacingLeavesPresent )
	{
		check(VertexInstanceUVs.GetNumChannels() == 8);

		// anchor
		if( RenderState->m_bFacingLeavesPresent )
		{
			DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_POSITION, VertexIndex, Data );
		}
		else
		{
			DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_LEAF_ANCHOR_POINT, VertexIndex, Data );
		}
		VertexInstanceUVs.Set( VertexInstanceID, 4, FVector2f( VertexInstanceUVs.Get( VertexInstanceID, 4 ).X, -Data[ 0 ] ) );
		VertexInstanceUVs.Set( VertexInstanceID, 5, FVector2f( Data[ 1 ], Data[ 2 ] ) );

		// leaf wind
		DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_WIND_EXTRA_DATA, VertexIndex, Data );
		VertexInstanceUVs.Set( VertexInstanceID, 6, FVector2f( Data[ 0 ], Data[ 1 ] ) );
		VertexInstanceUVs.Set( VertexInstanceID, 7, FVector2f( Data[ 2 ], 0 ) );
		DrawCall->GetProperty( SpeedTree::VERTEX_PROPERTY_WIND_FLAGS, VertexIndex, Data );
		VertexInstanceUVs.Set( VertexInstanceID, 7, FVector2f( VertexInstanceUVs.Get( VertexInstanceID, 7 ).X, Data[ 0 ] ) );
	}
	return VertexInstanceID;
}


UObject* USpeedTreeImportFactory::FactoryCreateBinary7(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	FString MeshName = ObjectTools::SanitizeObjectName(InName.ToString());
	FString NewPackageName = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetName()) + TEXT("/") + MeshName;
	UPackage* Package = UPackageTools::FindOrCreatePackageForAssetType(FName(*NewPackageName), UStaticMesh::StaticClass());
	check(Package);
	NewPackageName = Package->GetFullName();
	MeshName = FPaths::GetBaseFilename(NewPackageName, true);

	UStaticMesh* ExistingMesh = FindObject<UStaticMesh>(Package, *MeshName);
	USpeedTreeImportData* ExistingImportData = nullptr;
	if (ExistingMesh)
	{
		//Grab the existing asset data to fill correctly the option with the original import value
		ExistingImportData = Cast<USpeedTreeImportData>(ExistingMesh->GetAssetImportData());
	}

	USpeedTreeImportData* SpeedTreeImportData = nullptr;
	if (IsAutomatedImport())
	{
		SpeedTreeImportData = GetAutomatedImportOptions(ExistingImportData);
	}
	else
	{
		TSharedPtr<SWindow> ParentWindow;
		// Check if the main frame is loaded.  When using the old main frame it may not be.
		if( FModuleManager::Get().IsModuleLoaded( "MainFrame" ) )
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
			ParentWindow = MainFrame.GetParentWindow();
		}

		TSharedPtr<SSpeedTreeImportOptions> Options;

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("WindowTitle", "SpeedTree Options" ))
				.SizingRule( ESizingRule::Autosized );

		Window->SetContent(SAssignNew(Options, SSpeedTreeImportOptions).WidgetWindow(Window).ReimportAssetData(ExistingImportData));

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

		SpeedTreeImportData = Options->SpeedTreeImportData;

		if (!Options->ShouldImport())
		{
			//If user cancel, set the boolean
			bOutOperationCanceled = true;
		}

	}
	
	UStaticMesh* StaticMesh = nullptr;

	if ( !bOutOperationCanceled )
	{
#ifdef SPEEDTREE_KEY
		SpeedTree::CCore::Authorize(UE_STRINGIZE(SPEEDTREE_KEY));
#endif
		
		SpeedTree::CCore SpeedTree;
		if (!SpeedTree.LoadTree(Buffer, BufferEnd - Buffer, false, false, SpeedTreeImportData->TreeScale))
		{
			UE_LOGF(LogSpeedTreeImport, Error, "%ls", ANSI_TO_TCHAR(SpeedTree.GetError( )));
		}
		else
		{
			FSpeedTreeImportContext ImportContext;

			const SpeedTree::SGeometry* SpeedTreeGeometry = SpeedTree.GetGeometry();
			if ((SpeedTreeImportData->ImportGeometryType == EImportGeometryType::IGT_Billboards && SpeedTreeGeometry->m_sVertBBs.m_nNumBillboards == 0) ||
				(SpeedTreeImportData->ImportGeometryType == EImportGeometryType::IGT_3D && SpeedTreeGeometry->m_nNumLods == 0))
			{
				UE_LOGF(LogSpeedTreeImport, Error, "Tree contains no useable geometry");
			}
			else
			{
				LoadedPackages.Empty( );

				TArray<FStaticMaterial> OldMaterials;
				FGlobalComponentReregisterContext RecreateComponents;
				// clear out old mesh
				if (ExistingMesh)
				{
					OldMaterials = UE::SpeedTreeImporter::Private::ClearOutOldMesh(*ExistingMesh);
				}
				
				StaticMesh = NewObject<UStaticMesh>(Package, FName(*MeshName), Flags | RF_Public);

				// Copy the speed tree import asset from the option windows
				if (StaticMesh->GetAssetImportData() == nullptr || !StaticMesh->GetAssetImportData()->IsA(USpeedTreeImportData::StaticClass()))
				{
					StaticMesh->SetAssetImportData(NewObject<USpeedTreeImportData>(StaticMesh, NAME_None));
				}

				check(SpeedTreeImportData);
				StaticMesh->GetAssetImportData()->Update(UFactory::GetCurrentFilename());
				Cast<USpeedTreeImportData>(StaticMesh->GetAssetImportData())->CopyFrom(SpeedTreeImportData);
				
				// clear out any old data
				StaticMesh->SetNumSourceModels(0);
				StaticMesh->GetSectionInfoMap().Clear();
				StaticMesh->GetOriginalSectionInfoMap().Clear();
				StaticMesh->GetStaticMaterials().Empty();

				// Lightmap data
				StaticMesh->SetLightingGuid();
				StaticMesh->SetLightMapResolution(128);
				StaticMesh->SetLightMapCoordinateIndex(1);

				// set up SpeedTree wind data
				if (!StaticMesh->SpeedTreeWind.IsValid())
					StaticMesh->SpeedTreeWind = TSharedPtr<FSpeedTreeWind>(new FSpeedTreeWind);
				const SpeedTree::CWind* Wind = &(SpeedTree.GetWind());
				CopySpeedTreeWind7(Wind, StaticMesh->SpeedTreeWind);

				// choose wind type based on options enabled
				ESpeedTreeWindType WindType = STW_None;
				if (SpeedTreeImportData->IncludeWindCheck && Wind->IsOptionEnabled(SpeedTree::CWind::GLOBAL_WIND))
				{
					WindType = STW_Fastest;

					if (Wind->IsOptionEnabled(SpeedTree::CWind::BRANCH_DIRECTIONAL_FROND_1))
					{
						WindType = STW_Palm;
					}
					else if (Wind->IsOptionEnabled(SpeedTree::CWind::LEAF_TUMBLE_1))
					{
						WindType = STW_Best;
					}
					else if (Wind->IsOptionEnabled(SpeedTree::CWind::BRANCH_SIMPLE_1))
					{
						WindType = STW_Better;
					}
					else if (Wind->IsOptionEnabled(SpeedTree::CWind::LEAF_RIPPLE_VERTEX_NORMAL_1))
					{
						WindType = STW_Fast;
					}
				}

				// Force LOD code out of the shaders if we only have one LOD
				if (SpeedTreeImportData->IncludeSmoothLODCheck)
				{
					int32 TotalLODs = 0;
					if (SpeedTreeImportData->ImportGeometryType != EImportGeometryType::IGT_Billboards)
					{
						TotalLODs += SpeedTreeGeometry->m_nNumLods;
					}
					if (SpeedTreeImportData->ImportGeometryType != EImportGeometryType::IGT_3D && SpeedTreeGeometry->m_sVertBBs.m_nNumBillboards > 0)
					{
						++TotalLODs;
					}
					if (TotalLODs < 2)
					{
						SpeedTreeImportData->IncludeSmoothLODCheck = !SpeedTreeImportData->IncludeSmoothLODCheck;
					}
				}

				// make geometry LODs
				if (SpeedTreeImportData->ImportGeometryType != EImportGeometryType::IGT_Billboards)
				{
					int32 BranchMaterialsMade = 0;
					int32 FrondMaterialsMade = 0;
					int32 LeafMaterialsMade = 0;
					int32 FacingLeafMaterialsMade = 0;
					int32 MeshMaterialsMade = 0;
					TMap<int32, int32> RenderStateIndexToStaticMeshIndex;
				
					for (int32 LODIndex = 0; LODIndex < SpeedTreeGeometry->m_nNumLods; ++LODIndex)
					{
						const SpeedTree::SLod* TreeLOD = &SpeedTreeGeometry->m_pLods[LODIndex];
						FStaticMeshSourceModel& LODModel = StaticMesh->AddSourceModel();
						FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(LODIndex);
						FStaticMeshAttributes Attributes(*MeshDescription);
						
						TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
						TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
						TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
						TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
						TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
						TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
						TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
						TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

						// compute the number of texcoords we need so we can pad when necessary
						int32 NumUVs = 7; // static meshes have fewer, but they are so rare, we shouldn't complicate things for them
						for (int32 DrawCallIndex = 0; DrawCallIndex < TreeLOD->m_nNumDrawCalls; ++DrawCallIndex)
						{
							const SpeedTree::SDrawCall* DrawCall = &TreeLOD->m_pDrawCalls[DrawCallIndex];
							const SpeedTree::SRenderState* RenderState = DrawCall->m_pRenderState;
							if (RenderState->m_bLeavesPresent || RenderState->m_bFacingLeavesPresent)
							{
								NumUVs = FMath::Max(8, NumUVs);
							}
						}
						//Speedtree use UVs to store is data
						VertexInstanceUVs.SetNumChannels(NumUVs);

						TMap<int32, FPolygonGroupID> MaterialToPolygonGroup;
						MaterialToPolygonGroup.Reserve(TreeLOD->m_nNumDrawCalls);

						for (int32 DrawCallIndex = 0; DrawCallIndex < TreeLOD->m_nNumDrawCalls; ++DrawCallIndex)
						{
							SpeedTree::st_float32 Data[4];
							const SpeedTree::SDrawCall* DrawCall = &TreeLOD->m_pDrawCalls[DrawCallIndex];
							const SpeedTree::SRenderState* RenderState = DrawCall->m_pRenderState;

							// make material for this render state, if needed
							int32 MaterialIndex = -1;
							int32* OldMaterial = RenderStateIndexToStaticMeshIndex.Find(DrawCall->m_nRenderStateIndex);
							if (OldMaterial == NULL)
							{
								FString MaterialName = MeshName;

								if (RenderState->m_bBranchesPresent)
								{
									MaterialName += "_Branches";
									if (BranchMaterialsMade > 0)
										MaterialName += FString::Printf(TEXT("_%d"), BranchMaterialsMade + 1);
									++BranchMaterialsMade;
								}
								else if (RenderState->m_bFrondsPresent)
								{
									MaterialName += "_Fronds";
									if (FrondMaterialsMade > 0)
										MaterialName += FString::Printf(TEXT("_%d"), FrondMaterialsMade + 1);
									++FrondMaterialsMade;
								}
								else if (RenderState->m_bFacingLeavesPresent)
								{
									MaterialName += "_FacingLeaves";
									if (FacingLeafMaterialsMade > 0)
										MaterialName += FString::Printf(TEXT("_%d"), FacingLeafMaterialsMade + 1);
									++FacingLeafMaterialsMade;
								}
								else if (RenderState->m_bLeavesPresent)
								{
									MaterialName += "_Leaves";
									if (LeafMaterialsMade > 0)
										MaterialName += FString::Printf(TEXT("_%d"), LeafMaterialsMade + 1);
									++LeafMaterialsMade;
								}
								else if (RenderState->m_bRigidMeshesPresent)
								{
									MaterialName += "_Meshes";
									if (MeshMaterialsMade > 0)
										MaterialName += FString::Printf(TEXT("_%d"), MeshMaterialsMade + 1);
									++MeshMaterialsMade;
								}
								else if (RenderState->m_bHorzBillboard || RenderState->m_bVertBillboard)
								{
									MaterialName += "_Billboards";
								}

								MaterialName = ObjectTools::SanitizeObjectName(MaterialName);

								UMaterialInterface* Material = CreateSpeedTreeMaterial7(InParent, MaterialName, RenderState, SpeedTreeImportData, WindType, SpeedTreeGeometry->m_sVertBBs.m_nNumBillboards, LoadedPackages, ImportContext);
								
								RenderStateIndexToStaticMeshIndex.Add(DrawCall->m_nRenderStateIndex, StaticMesh->GetStaticMaterials().Num());
								MaterialIndex = StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material, FName(*MaterialName), FName(*MaterialName)));
							}
							else
							{
								MaterialIndex = *OldMaterial;
							}

							const FPolygonGroupID* CurrentPolygonGroupIDPtr = MaterialToPolygonGroup.Find(MaterialIndex);
							if (!CurrentPolygonGroupIDPtr)
							{
								const FPolygonGroupID PolygonGroupID = MeshDescription->CreatePolygonGroup();
								PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = StaticMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName;
								CurrentPolygonGroupIDPtr = &MaterialToPolygonGroup.Add(MaterialIndex, PolygonGroupID);
							}

							FPolygonGroupID CurrentPolygonGroupID = *CurrentPolygonGroupIDPtr;

							int32 IndexOffset = VertexPositions.GetNumElements();

							for (int32 VertexIndex = 0; VertexIndex < DrawCall->m_nNumVertices; ++VertexIndex)
							{
								// position
								DrawCall->GetProperty(SpeedTree::VERTEX_PROPERTY_POSITION, VertexIndex, Data);

								if (RenderState->m_bFacingLeavesPresent)
								{
									SpeedTree::st_float32 Data2[4];
									DrawCall->GetProperty(SpeedTree::VERTEX_PROPERTY_LEAF_CARD_CORNER, VertexIndex, Data2);
									Data[0] -= Data2[0];
									Data[1] += Data2[1];
									Data[2] += Data2[2];                                    
								}
								FVertexID VertexID = MeshDescription->CreateVertex();
								VertexPositions[VertexID] = FVector3f(-Data[0], Data[1], Data[2]);
							}

							const SpeedTree::st_byte* pIndexData = &*DrawCall->m_pIndexData;
							const SpeedTree::st_uint32* Indices32 = (SpeedTree::st_uint32*)pIndexData;
							const SpeedTree::st_uint16* Indices16 = (SpeedTree::st_uint16*)pIndexData;

							int32 TriangleCount = DrawCall->m_nNumIndices / 3;

							//Create all vertex instance
							for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
							{
								TArray<FVertexInstanceID> CornerVertexInstanceIDs;
								CornerVertexInstanceIDs.SetNum(3);
								FVertexID CornerVertexIDs[3];
								for (int32 Corner = 0; Corner < 3; ++Corner)
								{
									//Create the vertex instances
									CornerVertexInstanceIDs[Corner] = ProcessTriangleCorner( SpeedTree, TriangleIndex, Corner, DrawCall, Indices32, Indices16, *MeshDescription, IndexOffset, NumUVs, RenderState,
										VertexInstanceNormals, VertexInstanceTangents, VertexInstanceBinormalSigns, VertexInstanceColors, VertexInstanceUVs);

									CornerVertexIDs[Corner] = MeshDescription->GetVertexInstanceVertex(CornerVertexInstanceIDs[Corner]);
								}

								// Insert a polygon into the mesh
								MeshDescription->CreatePolygon(CurrentPolygonGroupID, CornerVertexInstanceIDs);
							}
						}

						LODModel.BuildSettings.bRecomputeNormals = false;
						LODModel.BuildSettings.bRecomputeTangents = false;
						LODModel.BuildSettings.bRemoveDegenerates = true;
						LODModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
						LODModel.BuildSettings.bUseFullPrecisionUVs = false;
						LODModel.BuildSettings.bGenerateLightmapUVs = false;
						LODModel.ScreenSize.Default = 0.1f / FMath::Pow(2.0f, StaticMesh->GetNumSourceModels() - 1);
						StaticMesh->CommitMeshDescription(LODIndex);

						for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++MaterialIndex)
						{
							FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(LODIndex, MaterialIndex);
							Info.MaterialIndex = MaterialIndex;
							StaticMesh->GetSectionInfoMap().Set(LODIndex, MaterialIndex, Info);
						}
						StaticMesh->GetOriginalSectionInfoMap().CopyFrom(StaticMesh->GetSectionInfoMap());
					}
				}

				// make billboard LOD
				if (SpeedTreeImportData->ImportGeometryType != EImportGeometryType::IGT_3D && SpeedTreeGeometry->m_sVertBBs.m_nNumBillboards > 0)
				{
					FStaticMeshSourceModel& LODModel = StaticMesh->AddSourceModel();
					const int32 LODIndex = StaticMesh->GetNumSourceModels() - 1;
					FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(LODIndex);
					FStaticMeshAttributes Attributes(*MeshDescription);

					TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
					TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
					TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
					TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
					TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
					TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
					TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
					TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

					//Speedtree use UVs to store is data
					VertexInstanceUVs.SetNumChannels(2);

					FString MaterialName = MeshName + "_Billboard";
					UMaterialInterface* Material = CreateSpeedTreeMaterial7(InParent, MaterialName, &SpeedTreeGeometry->m_aBillboardRenderStates[SpeedTree::RENDER_PASS_MAIN], SpeedTreeImportData, WindType, SpeedTreeGeometry->m_sVertBBs.m_nNumBillboards, LoadedPackages, ImportContext);
					int32 MaterialIndex = StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material, FName(*MaterialName), FName(*MaterialName)));

					const FPolygonGroupID CurrentPolygonGroupID = MeshDescription->CreatePolygonGroup();
					PolygonGroupImportedMaterialSlotNames[CurrentPolygonGroupID] = StaticMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName;
					
					// fill out triangles
					float BillboardWidth = SpeedTreeGeometry->m_sVertBBs.m_fWidth;
					float BillboardBottom = SpeedTreeGeometry->m_sVertBBs.m_fBottomPos;
					float BillboardTop = SpeedTreeGeometry->m_sVertBBs.m_fTopPos;
					float BillboardHeight = BillboardTop - BillboardBottom;

					// data for a regular billboard quad
					const SpeedTree::st_uint16 BillboardQuadIndices[] = { 0, 1, 2, 0, 2, 3 };
					const SpeedTree::st_float32 BillboardQuadVertices[] = 
					{
						1.0f, 1.0f,
						0.0f, 1.0f,
						0.0f, 0.0f, 
						1.0f, 0.0f
					};

					// choose between quad or compiler-generated cutout
					int32 NumVertices = SpeedTreeGeometry->m_sVertBBs.m_nNumCutoutVertices;
					const SpeedTree::st_float32* Vertices = SpeedTreeGeometry->m_sVertBBs.m_pCutoutVertices;
					int32 NumIndices = SpeedTreeGeometry->m_sVertBBs.m_nNumCutoutIndices;
					const SpeedTree::st_uint16* Indices = SpeedTreeGeometry->m_sVertBBs.m_pCutoutIndices;
					if (NumIndices == 0)
					{
						NumVertices = 4;
						Vertices = BillboardQuadVertices;
						NumIndices = 6;
						Indices = BillboardQuadIndices;
					}

					// make the billboards
					for (int32 BillboardIndex = 0; BillboardIndex < SpeedTreeGeometry->m_sVertBBs.m_nNumBillboards; ++BillboardIndex)
					{
						FRotator Facing(0, 90.0f - 360.0f * (float)BillboardIndex / (float)SpeedTreeGeometry->m_sVertBBs.m_nNumBillboards, 0);
						FRotationMatrix BillboardRotate(Facing);

						FVector TangentX = BillboardRotate.TransformVector(FVector(1.0f, 0.0f, 0.0f));
						FVector TangentY = BillboardRotate.TransformVector(FVector(0.0f, 0.0f, -1.0f));
						FVector TangentZ = BillboardRotate.TransformVector(FVector(0.0f, 1.0f, 0.0f));
	
						const float* TexCoords = &SpeedTreeGeometry->m_sVertBBs.m_pTexCoords[BillboardIndex * 4];
						bool bRotated = (SpeedTreeGeometry->m_sVertBBs.m_pRotated[BillboardIndex] == 1);

						int32 IndexOffset = VertexPositions.GetNumElements();
					
						// position
						for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
						{
							const SpeedTree::st_float32* Vertex = &Vertices[VertexIndex * 2];
							FVector Position = BillboardRotate.TransformVector(FVector(Vertex[0] * BillboardWidth - BillboardWidth * 0.5f, 0.0f, Vertex[1] * BillboardHeight + BillboardBottom));
							
							FVertexID VertexID = MeshDescription->CreateVertex();
							VertexPositions[VertexID] = FVector3f(Position);
						}

						// other data
						int32 NumTriangles = NumIndices / 3;
						for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
						{
							TArray<FVertexInstanceID> CornerVertexInstanceIDs;
							CornerVertexInstanceIDs.SetNum(3);
							FVertexID CornerVertexIDs[3];
							for (int32 Corner = 0; Corner < 3; ++Corner)
							{
								int32 Index = Indices[TriangleIndex * 3 + Corner];
								const SpeedTree::st_float32* Vertex = &Vertices[Index * 2];

								FVertexID VertexID(IndexOffset + Index);
								const FVertexInstanceID& VertexInstanceID = MeshDescription->CreateVertexInstance(VertexID);

								VertexInstanceTangents[VertexInstanceID] = (FVector3f)TangentX;
								VertexInstanceNormals[VertexInstanceID] = (FVector3f)TangentZ;
								VertexInstanceBinormalSigns[VertexInstanceID] = GetBasisDeterminantSign(TangentX.GetSafeNormal(), TangentY.GetSafeNormal(), TangentZ.GetSafeNormal());
								if (bRotated)
								{
									VertexInstanceUVs.Set(VertexInstanceID, 0, FVector2f(TexCoords[0] + Vertex[1] * TexCoords[2], TexCoords[1] + Vertex[0] * TexCoords[3]));
								}
								else
								{
									VertexInstanceUVs.Set(VertexInstanceID, 0, FVector2f(TexCoords[0] + Vertex[0] * TexCoords[2], TexCoords[1] + Vertex[1] * TexCoords[3]));
								}

								// lightmap coord
								VertexInstanceUVs.Set(VertexInstanceID, 1, VertexInstanceUVs.Get(VertexInstanceID, 0));

								CornerVertexInstanceIDs[Corner] = VertexInstanceID;
								CornerVertexIDs[Corner] = VertexID;
							}

							// Insert a polygon into the mesh
							MeshDescription->CreatePolygon(CurrentPolygonGroupID, CornerVertexInstanceIDs);
						}
					}

					LODModel.BuildSettings.bRecomputeNormals = false;
					LODModel.BuildSettings.bRecomputeTangents = false;
					LODModel.BuildSettings.bRemoveDegenerates = true;
					LODModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
					LODModel.BuildSettings.bUseFullPrecisionUVs = false;
					LODModel.BuildSettings.bGenerateLightmapUVs = false;
					LODModel.ScreenSize.Default = 0.1f / FMath::Pow(2.0f, StaticMesh->GetNumSourceModels() - 1);
					StaticMesh->CommitMeshDescription(LODIndex);
					// Add mesh section info entry for billboard LOD (only one section/material index)
					FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(LODIndex, 0);
					Info.MaterialIndex = MaterialIndex;
					StaticMesh->GetSectionInfoMap().Set(LODIndex, 0, Info);
					StaticMesh->GetOriginalSectionInfoMap().Set(LODIndex, MaterialIndex, Info);
				}

				if (OldMaterials.Num() == StaticMesh->GetStaticMaterials().Num())
				{
					StaticMesh->SetStaticMaterials(OldMaterials);
				}

				//Set the Imported version before calling the build
				StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

				StaticMesh->Build();

				if (SpeedTreeImportData->IncludeCollision)
				{
					int32 NumCollisionObjects = 0;
					const SpeedTree::SCollisionObject* CollisionObjects = SpeedTree.GetCollisionObjects(NumCollisionObjects);
					if (CollisionObjects != NULL && NumCollisionObjects > 0)
					{
						MakeBodyFromCollisionObjects7(StaticMesh, CollisionObjects, NumCollisionObjects);
					}
				}

				// make better LOD info for SpeedTrees
				if (SpeedTreeImportData->LODType == EImportLODType::ILT_IndividualActors)
				{
					StaticMesh->SetAutoComputeLODScreenSize(false);
				}
				StaticMesh->SetRequiresLODDistanceConversion(false);
			}
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, StaticMesh);

	return StaticMesh;
}

UObject* USpeedTreeImportFactory::FactoryCreateBinary8(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	TSharedPtr<SWindow> ParentWindow;
	// Check if the main frame is loaded.  When using the old main frame it may not be.
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FString MeshName = ObjectTools::SanitizeObjectName(InName.ToString());
	FString NewPackageName = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetName()) + TEXT("/") + MeshName;
	UPackage* Package = UPackageTools::FindOrCreatePackageForAssetType(FName(*NewPackageName), UStaticMesh::StaticClass());
	check(Package);
	NewPackageName = Package->GetFullName();

	UStaticMesh* StaticMesh = NULL;

	GameEngine8::CTree SpeedTree;
	if (!SpeedTree.LoadFromData(Buffer, BufferEnd - Buffer))
	{
		UE_LOGF(LogSpeedTreeImport, Error, "Not a SpeedTree file");
	}
	else
	{
		LoadedPackages.Empty();

		FSpeedTreeImportContext ImportContext;
		UStaticMesh* ExistingMesh = FindObject<UStaticMesh>(Package, *MeshName);

		// Options
		USpeedTreeImportData* SpeedTreeImportData = nullptr;
		if (ExistingMesh)
		{
			//Grab the existing asset data to fill correctly the option with the original import value
			SpeedTreeImportData = Cast<USpeedTreeImportData>(ExistingMesh->GetAssetImportData());
		}

		bool bIsAutomatedImport = IsAutomatedImport();

		// clear out old mesh
		TArray<FStaticMaterial> OldMaterials;
		FGlobalComponentReregisterContext RecreateComponents;
		if (ExistingMesh)
		{
			OldMaterials = UE::SpeedTreeImporter::Private::ClearOutOldMesh(*ExistingMesh);
			StaticMesh = ExistingMesh;
		}

		if (!bIsAutomatedImport && !SpeedTreeImportData)
		{
			TSharedPtr<SSpeedTreeImportOptions> Options;

			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(LOCTEXT("WindowTitle", "SpeedTree Options"))
				.SizingRule(ESizingRule::Autosized);

			Window->SetContent(SAssignNew(Options, SSpeedTreeImportOptions).WidgetWindow(Window).ReimportAssetData(SpeedTreeImportData));

			FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

			if (!Options->ShouldImport())
			{
				bOutOperationCanceled = true;
				return nullptr;
			}
			
			SpeedTreeImportData = Options->SpeedTreeImportData;
		}

		if (bIsAutomatedImport)
		{
			SpeedTreeImportData = GetAutomatedImportOptions(SpeedTreeImportData);
		}

		if (!StaticMesh)
		{
			StaticMesh = NewObject<UStaticMesh>(Package, FName(*MeshName), Flags | RF_Public);
		}

		// Copy the speed tree import asset from the option windows
		if (StaticMesh->GetAssetImportData() == nullptr || !StaticMesh->GetAssetImportData()->IsA(USpeedTreeImportData::StaticClass()))
		{
			StaticMesh->SetAssetImportData(NewObject<USpeedTreeImportData>(StaticMesh, NAME_None));
		}

		check(SpeedTreeImportData);
		StaticMesh->GetAssetImportData()->Update(UFactory::GetCurrentFilename());
		Cast<USpeedTreeImportData>(StaticMesh->GetAssetImportData())->CopyFrom(SpeedTreeImportData);

		// clear out any old data
		StaticMesh->GetSectionInfoMap().Clear();
		StaticMesh->GetStaticMaterials().Empty();
		if (StaticMesh->GetNumSourceModels() != SpeedTree.Lods().Count())
		{
			StaticMesh->SetNumSourceModels(0);
			float Denominator = 1.0f / FMath::Max(1.0f, SpeedTree.Lods().Count() - 1.0f);

			for (uint32 LODIndex = 0; LODIndex < SpeedTree.Lods().Count(); ++LODIndex)
			{
				FStaticMeshSourceModel& LODModel = StaticMesh->AddSourceModel();
				LODModel.BuildSettings.SrcLightmapIndex = 1;
				LODModel.BuildSettings.DstLightmapIndex = 1;
				LODModel.BuildSettings.bRecomputeNormals = false;
				LODModel.BuildSettings.bRecomputeTangents = false;
				LODModel.BuildSettings.bRemoveDegenerates = true;
				LODModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
				LODModel.BuildSettings.bUseFullPrecisionUVs = false;
				LODModel.BuildSettings.bGenerateLightmapUVs = false;

				if (SpeedTreeImportData->LODType == ILT_IndividualActors)
				{
					LODModel.ScreenSize = FMath::Lerp(1.0f, 0.1f, FMath::Square(LODIndex * Denominator));
				}
				else
				{
					LODModel.ScreenSize = FMath::Lerp(1.0f, 0.25f, LODIndex * Denominator);
				}
			}
		}

		// Lightmap data
		StaticMesh->SetLightingGuid();
		StaticMesh->SetLightMapResolution(SpeedTree.LightmapSize());
		StaticMesh->SetLightMapCoordinateIndex(1);

		// set up SpeedTree wind data
		if (!StaticMesh->SpeedTreeWind.IsValid())
			StaticMesh->SpeedTreeWind = TSharedPtr<FSpeedTreeWind>(new FSpeedTreeWind);
		const GameEngine8::SWindConfig* Wind = &(SpeedTree.Wind());
		CopySpeedTreeWind8(Wind, StaticMesh->SpeedTreeWind);

		ESpeedTreeWindType WindType = STW_None;
		switch (Wind->m_ePreset)
		{
		case GameEngine8::SWindConfig::WIND_PRESET_FASTEST:	WindType = STW_Fastest; break;
		case GameEngine8::SWindConfig::WIND_PRESET_FAST:		WindType = STW_Fast; break;
		case GameEngine8::SWindConfig::WIND_PRESET_BETTER:	WindType = STW_Better; break;
		case GameEngine8::SWindConfig::WIND_PRESET_BEST:		WindType = STW_BestPlus; break;
		case GameEngine8::SWindConfig::WIND_PRESET_PALM:		WindType = STW_Palm; break;
		default: break;
		};

		// materials
		TMap<FIntPoint, int32> MaterialMap;

		// gui info
		StaticMesh->SetAutoComputeLODScreenSize(false);
		StaticMesh->SetRequiresLODDistanceConversion(false);
		bool bCrossfadeLOD = (SpeedTreeImportData->LODType == ILT_PaintedFoliage) ||
							((SpeedTree.Lods().Count() == 2) && SpeedTree.LastLodIsBillboard());

		// make geometry LODs
		for (uint32 LODIndex = 0; LODIndex < SpeedTree.Lods().Count(); ++LODIndex)
		{
			GameEngine8::CLod LOD = SpeedTree.Lods()[LODIndex];
			
			FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(LODIndex);
			FStaticMeshAttributes Attributes(*MeshDescription);

			TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
			TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
			TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
			TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
			TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
			TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

			//Speedtree use 8 UVs to store is data
			VertexInstanceUVs.SetNumChannels(8);

			for (uint32 VertexIndex = 0; VertexIndex < LOD.Vertices().Count(); ++VertexIndex)
			{
				const GameEngine8::SVertex& Vertex = LOD.Vertices()[VertexIndex];
				FVector vPosition = FVector(Vertex.m_vAnchor.x, Vertex.m_vAnchor.y, Vertex.m_vAnchor.z) + FVector(Vertex.m_vOffset.x, Vertex.m_vOffset.y, Vertex.m_vOffset.z);
				FVertexID VertexID = MeshDescription->CreateVertex();
				VertexPositions[VertexID] = FVector3f(vPosition);
			}

			// Per-LOD material -> polygon group mapping
			TMap<FIntPoint, FPolygonGroupID> PolygonGroupIDMap;
			PolygonGroupIDMap.Reserve(LOD.DrawCalls().Count());

			for (uint32 DrawCallIndex = 0; DrawCallIndex < LOD.DrawCalls().Count(); ++DrawCallIndex)
			{
				const GameEngine8::SDrawCall& DrawCall = LOD.DrawCalls()[DrawCallIndex];

				// find correct material/geometry combo
				FIntPoint MaterialKey(DrawCall.m_uiMaterialIndex, DrawCall.m_eWindGeometryType);
				if (!MaterialMap.Contains(MaterialKey))
				{
					ESpeedTreeGeometryType GeomType;
					FString GeomString;
					#define HANDLE_WIND_TYPE(a) case GameEngine8::a: GeomType = STG_##a; GeomString = "_"#a; break
					switch (DrawCall.m_eWindGeometryType)
					{
					HANDLE_WIND_TYPE(Frond);
					HANDLE_WIND_TYPE(Leaf);
					HANDLE_WIND_TYPE(FacingLeaf);
					HANDLE_WIND_TYPE(Billboard);
					default: 
					HANDLE_WIND_TYPE(Branch);
					};

					GameEngine8::CMaterial SpeedTreeMaterial = SpeedTree.Materials()[DrawCall.m_uiMaterialIndex];
					FString MaterialName = FString(SpeedTreeMaterial.Name().Data());
					MaterialName.InsertAt(MaterialName.Len() - 4, GeomString);
					UMaterialInterface* Material = CreateSpeedTreeMaterial8(InParent, MaterialName, SpeedTreeMaterial, SpeedTreeImportData, WindType, GeomType, LoadedPackages, bCrossfadeLOD, ImportContext);
					MaterialMap.Add(MaterialKey, StaticMesh->GetStaticMaterials().Num());
					StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material, FName(*MaterialName), FName(*MaterialName)));
				}

				const int32 MaterialIndex = MaterialMap[MaterialKey];

				const FPolygonGroupID* CurrentPolygonGroupIDPtr = PolygonGroupIDMap.Find(MaterialKey);
				// If this LOD doesn't already have a polygon group for this material create one
				if (!CurrentPolygonGroupIDPtr)
				{
					const FPolygonGroupID PolygonGroupID = MeshDescription->CreatePolygonGroup();
					CurrentPolygonGroupIDPtr = &PolygonGroupIDMap.Add(MaterialKey, PolygonGroupID);
					PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = StaticMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName;
				}

				const FPolygonGroupID CurrentPolygonGroupID = *CurrentPolygonGroupIDPtr;

				FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(LODIndex, DrawCallIndex);
				Info.MaterialIndex = MaterialIndex;
				StaticMesh->GetSectionInfoMap().Set(LODIndex, DrawCallIndex, Info);

				int32 TriangleCount = DrawCall.m_uiIndexCount / 3;
				for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
				{
					TArray<FVertexInstanceID> CornerVertexInstanceIDs;
					CornerVertexInstanceIDs.SetNum(3);
					FVertexID CornerVertexIDs[3];
					for (int32 Corner = 0; Corner < 3; ++Corner)
					{
						int32 VertexIndex = LOD.Indices()[DrawCall.m_uiIndexStart + TriangleIndex * 3 + (2 - Corner)];
						const GameEngine8::SVertex& Vertex = LOD.Vertices()[VertexIndex];

						FVertexID VertexID(VertexIndex);
						const FVertexInstanceID& VertexInstanceID = MeshDescription->CreateVertexInstance(VertexID);

						// tangents
						FVector TangentX(Vertex.m_vTangent.x, Vertex.m_vTangent.y, Vertex.m_vTangent.z);
						FVector TangentY(-Vertex.m_vBinormal.x, -Vertex.m_vBinormal.y, -Vertex.m_vBinormal.z);
						FVector TangentZ(Vertex.m_vNormal.x, Vertex.m_vNormal.y, Vertex.m_vNormal.z);

						VertexInstanceTangents[VertexInstanceID] = (FVector3f)TangentX;
						VertexInstanceNormals[VertexInstanceID] = (FVector3f)TangentZ;
						VertexInstanceBinormalSigns[VertexInstanceID] = GetBasisDeterminantSign(TangentX.GetSafeNormal(), TangentY.GetSafeNormal(), TangentZ.GetSafeNormal());
						
						// ao and branch blend in vertex color
						if (DrawCall.m_eWindGeometryType != GameEngine8::Billboard)
						{
							uint8 AO = Vertex.m_fAmbientOcclusion * 255.0f;
							VertexInstanceColors[VertexInstanceID] = FLinearColor(FColor(AO, AO, AO, Vertex.m_fBlendWeight * 255));
						}

						// All texcoords are packed into 4 float4 vertex attributes
						// Data is as follows								

						//		Branches			Fronds				Leaves				Billboards
						//
						// 0	Diffuse				Diffuse				Diffuse				Diffuse			
						// 1	Lightmap UV			Lightmap UV			Lightmap UV			Lightmap UV		
						// 2	Branch Wind XY		Branch Wind XY		Branch Wind XY		Top-Down billboard, 0
						// 3	LOD XY				LOD XY				LOD XY				
						// 4	LOD Z, Seam Amount	LOD Z, 0			LOD Z, Anchor X		
						// 5	Detail UV			Frond Wind XY		Anchor YZ	
						// 6	Seam UV				Frond Wind Z, 0		Leaf Wind XY
						// 7	0					0					Leaf Wind Z, Leaf Group

						// diffuse
						VertexInstanceUVs.Set(VertexInstanceID, 0, FVector2f(Vertex.m_vTexCoord.x, Vertex.m_vTexCoord.y));

						// lightmap
						VertexInstanceUVs.Set(VertexInstanceID, 1, FVector2f(Vertex.m_vLightmapTexCoord.x, Vertex.m_vLightmapTexCoord.y));

						if (DrawCall.m_eWindGeometryType == GameEngine8::Billboard)
						{
							VertexInstanceUVs.Set(VertexInstanceID, 2, FVector2f((Vertex.m_vNormal.z > 0.5f) ? 1.0f : 0.0f, 0.0f));
						}
						else
						{
							// branch wind
							VertexInstanceUVs.Set(VertexInstanceID, 2, FVector2f(Vertex.m_vWindBranch.x, Vertex.m_vWindBranch.y));

							// lod
							FVector vLodPosition = FVector(Vertex.m_vAnchor.x, Vertex.m_vAnchor.y, Vertex.m_vAnchor.z) +
								FVector(Vertex.m_vLodOffset.x, Vertex.m_vLodOffset.y, Vertex.m_vLodOffset.z);
							VertexInstanceUVs.Set(VertexInstanceID, 3, FVector2f(vLodPosition[0], vLodPosition[1]));
							VertexInstanceUVs.Set(VertexInstanceID, 4, FVector2f(vLodPosition[2], 0.0f));

							// other
							if (DrawCall.m_eWindGeometryType == GameEngine8::Branch)
							{
								// detail (not used in v8)
								VertexInstanceUVs.Set(VertexInstanceID, 5, FVector2f(0.0f, 0.0f));

								// branch seam
								VertexInstanceUVs.Set(VertexInstanceID, 6, FVector2f(0.0f, 0.0f));
								VertexInstanceUVs.Set(VertexInstanceID, 4, FVector2f(VertexInstanceUVs.Get(VertexInstanceID, 4).X, Vertex.m_fBlendWeight));

								// keep alignment
								VertexInstanceUVs.Set(VertexInstanceID, 7, FVector2f(0.0f, 0.0f));
							}
							else if (DrawCall.m_eWindGeometryType == GameEngine8::Frond)
							{
								// frond wind
								VertexInstanceUVs.Set(VertexInstanceID, 5, FVector2f(Vertex.m_vWindNonBranch.x, Vertex.m_vWindNonBranch.y));
								VertexInstanceUVs.Set(VertexInstanceID, 6, FVector2f(Vertex.m_vWindNonBranch.z, 0.0f));

								// keep alignment
								VertexInstanceUVs.Set(VertexInstanceID, 7, FVector2f(0.0f, 0.0f));
							}
							else if (DrawCall.m_eWindGeometryType == GameEngine8::Leaf || DrawCall.m_eWindGeometryType == GameEngine8::FacingLeaf)
							{
								// anchor
								VertexInstanceUVs.Set(VertexInstanceID, 4, FVector2f(VertexInstanceUVs.Get(VertexInstanceID, 4).X, Vertex.m_vAnchor.x));
								VertexInstanceUVs.Set(VertexInstanceID, 5, FVector2f(Vertex.m_vAnchor.y, Vertex.m_vAnchor.z));

								// leaf wind
								VertexInstanceUVs.Set(VertexInstanceID, 6, FVector2f(Vertex.m_vWindNonBranch.x, Vertex.m_vWindNonBranch.y));
								VertexInstanceUVs.Set(VertexInstanceID, 7, FVector2f(Vertex.m_vWindNonBranch.z, (Vertex.m_bWindLeaf2Flag ? 1.0f : 0.0f)));
							}
						}
						CornerVertexInstanceIDs[Corner] = VertexInstanceID;
						CornerVertexIDs[Corner] = VertexID;
					}

					// Insert a polygon into the mesh
					MeshDescription->CreatePolygon(CurrentPolygonGroupID, CornerVertexInstanceIDs);
				}
			}

			//Save the created mesh
			StaticMesh->CommitMeshDescription(LODIndex);
		}

		// replace materials if they've been switched out
		if (OldMaterials.Num() == StaticMesh->GetStaticMaterials().Num())
		{
			StaticMesh->SetStaticMaterials(OldMaterials);
		}

		//Set the Imported version before calling the build
		StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
		StaticMesh->Build();

		// collision objects
		if (SpeedTreeImportData->IncludeCollision)
		{
			MakeBodyFromCollisionObjects8(StaticMesh, SpeedTree.CollisionObjects());
		}
		else
		{
			StaticMesh->CreateBodySetup();
			StaticMesh->GetBodySetup()->RemoveSimpleCollision();
			StaticMesh->GetBodySetup()->ClearPhysicsMeshes();
			StaticMesh->GetBodySetup()->InvalidatePhysicsData();
			RefreshCollisionChange(*StaticMesh);
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, StaticMesh);

	return StaticMesh;
}

UObject* USpeedTreeImportFactory::FactoryCreateBinary9(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	if (InParent == nullptr)
	{
		return nullptr;
	}

	GEditor->SelectNone(false, false);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	FString MeshName = ObjectTools::SanitizeObjectName(InName.ToString());
	FString BasePackageName = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetName()) / MeshName;
	BasePackageName = UPackageTools::SanitizePackageName(BasePackageName);

	UStaticMesh* ExistingMesh = nullptr;
	UPackage* Package = nullptr;
	// First check if the asset already exists.
	{
		FString ObjectPath = BasePackageName + TEXT(".") + MeshName;
		ExistingMesh = LoadObject<UStaticMesh>(nullptr, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn);
	}

	if (ExistingMesh)
	{
		Package = ExistingMesh->GetOutermost();
	}
	else
	{
		const FString Suffix(TEXT(""));

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FString FinalPackageName;
		AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, Suffix, FinalPackageName, MeshName);

		Package = CreatePackage(*FinalPackageName);
	}

	USpeedTreeImportData* ExistingImportData = nullptr;
	if (ExistingMesh)
	{
		//Grab the existing asset data to fill correctly the option with the original import value
		ExistingImportData = Cast<USpeedTreeImportData>(ExistingMesh->GetAssetImportData());
	}

	UStaticMesh* StaticMesh = nullptr;

	GameEngine9::CTree SpeedTree;
	if (!SpeedTree.LoadFromData(Buffer, BufferEnd - Buffer))
	{
		UE_LOGF(LogSpeedTreeImport, Error, "Not a SpeedTree file");
	}
	else
	{
		LoadedPackages.Empty();

		FSpeedTreeImportContext ImportContext;

		// clear out old mesh
		TArray<FStaticMaterial> OldMaterials;
		FGlobalComponentReregisterContext RecreateComponents;
		if (ExistingMesh)
		{
			OldMaterials = UE::SpeedTreeImporter::Private::ClearOutOldMesh(*ExistingMesh);
			StaticMesh = ExistingMesh;
		}
		else
		{
			StaticMesh = NewObject<UStaticMesh>(Package, FName(*MeshName), Flags | RF_Public);
		}

		// Copy the speed tree import asset from the option windows
		if (StaticMesh->GetAssetImportData() == nullptr || !StaticMesh->GetAssetImportData()->IsA(USpeedTreeImportData::StaticClass()))
		{
			StaticMesh->SetAssetImportData(NewObject<USpeedTreeImportData>(StaticMesh, NAME_None));
		}
		StaticMesh->GetAssetImportData()->Update(UFactory::GetCurrentFilename());

		// clear out any old data
		StaticMesh->GetSectionInfoMap().Clear();
		StaticMesh->GetStaticMaterials().Empty();
		if (StaticMesh->GetNumSourceModels() != SpeedTree.Lods().Count())
		{
			StaticMesh->SetNumSourceModels(0);
			float Denominator = 1.0f / FMath::Max(1.0f, SpeedTree.Lods().Count() - 1.0f);

			for (uint32 LODIndex = 0; LODIndex < SpeedTree.Lods().Count(); ++LODIndex)
			{
				FStaticMeshSourceModel& LODModel = StaticMesh->AddSourceModel();
				LODModel.BuildSettings.SrcLightmapIndex = 1;
				LODModel.BuildSettings.DstLightmapIndex = 1;
				LODModel.BuildSettings.bRecomputeNormals = false;
				LODModel.BuildSettings.bRecomputeTangents = false;
				LODModel.BuildSettings.bRemoveDegenerates = true;
				LODModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
				LODModel.BuildSettings.bUseFullPrecisionUVs = false;
				LODModel.BuildSettings.bGenerateLightmapUVs = false;
				LODModel.ScreenSize = FMath::Lerp(1.0f, 0.25f, LODIndex * Denominator);
			}
		}

		// materials
		TMap<int32, int32> MaterialMap;

		// gui info
		StaticMesh->SetAutoComputeLODScreenSize(false);
		StaticMesh->SetRequiresLODDistanceConversion(false);
	
		// VB setup info	
		bool bHasBranch2Data = SpeedTree.Wind().DoBranch2( );
		bool bHasFacingData = false;
		for (uint32 LODIndex = 0; LODIndex < SpeedTree.Lods().Count(); ++LODIndex)
		{
			GameEngine9::CLod LOD = SpeedTree.Lods()[LODIndex];
			for (uint32 DrawCallIndex = 0; DrawCallIndex < LOD.DrawCalls().Count(); ++DrawCallIndex)
			{
				const GameEngine9::SDrawCall& DrawCall = LOD.DrawCalls()[DrawCallIndex];
				bHasFacingData |= DrawCall.m_bContainsFacingGeometry;
			}
		}
		
		uint32 NumUVs = 4;
		if (bHasBranch2Data)
		{
			NumUVs += 2;
		}
		if (bHasFacingData)
		{
			NumUVs += 2;
		}

		// Lightmap data
		StaticMesh->SetLightingGuid();
		StaticMesh->SetLightMapResolution(SpeedTree.LightmapSize());
		StaticMesh->SetLightMapCoordinateIndex(3);

		// make geometry LODs
		for (uint32 LODIndex = 0; LODIndex < SpeedTree.Lods().Count(); ++LODIndex)
		{
			GameEngine9::CLod LOD = SpeedTree.Lods()[LODIndex];

			FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(LODIndex);
			FStaticMeshAttributes Attributes(*MeshDescription);

			TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
			TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
			TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
			TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
			TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
			TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

			VertexInstanceUVs.SetNumChannels(NumUVs);

			for (int32 MatIndex = 0; MatIndex < StaticMesh->GetStaticMaterials().Num(); ++MatIndex)
			{
				const FPolygonGroupID& PolygonGroupID = MeshDescription->CreatePolygonGroup();
				PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = StaticMesh->GetStaticMaterials()[MatIndex].ImportedMaterialSlotName;
			}

			for (uint32 VertexIndex = 0; VertexIndex < LOD.Vertices().Count(); ++VertexIndex)
			{
				const GameEngine9::SVertex& Vertex = LOD.Vertices()[VertexIndex];
				FVector vPosition = FVector(Vertex.m_vAnchor.x, Vertex.m_vAnchor.y, Vertex.m_vAnchor.z) + FVector(Vertex.m_vOffset.x, Vertex.m_vOffset.y, Vertex.m_vOffset.z);
				FVertexID VertexID = MeshDescription->CreateVertex();
				VertexPositions[VertexID] = FVector3f(vPosition);
			}

			for (uint32 DrawCallIndex = 0; DrawCallIndex < LOD.DrawCalls().Count(); ++DrawCallIndex)
			{
				const GameEngine9::SDrawCall& DrawCall = LOD.DrawCalls()[DrawCallIndex];

				// find correct material
				if (!MaterialMap.Contains(DrawCall.m_uiMaterialIndex))
				{
					// you could pick different template materials based on SpeedTree.TexturePacker()
					bool bBillboard = (SpeedTree.BillboardInfo().LastLodIsBillboard() && (LODIndex == (SpeedTree.Lods().Count() - 1)));
					UMaterialInterface* Template = (bBillboard ? TemplateBillboardMaterial : TemplateMaterial);

					GameEngine9::CMaterial SpeedTreeMaterial = SpeedTree.Materials()[DrawCall.m_uiMaterialIndex];
					FString MaterialName = FString(SpeedTreeMaterial.Name().Data());
					UMaterialInterface* Material = CreateSpeedTreeMaterial9(InParent, MaterialName, SpeedTreeMaterial, Template, SpeedTree.Wind(), bHasFacingData, SpeedTree.BillboardInfo().SideViewCount(), LoadedPackages, ImportContext);
					MaterialMap.Add(DrawCall.m_uiMaterialIndex, StaticMesh->GetStaticMaterials().Num());
					int32 AddedMaterialIndex = StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material, FName(*MaterialName), FName(*MaterialName)));
					const FPolygonGroupID& PolygonGroupID = MeshDescription->CreatePolygonGroup();
					PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = StaticMesh->GetStaticMaterials()[AddedMaterialIndex].ImportedMaterialSlotName;
				}
				const int32 MaterialIndex = MaterialMap[DrawCall.m_uiMaterialIndex];
				const FPolygonGroupID CurrentPolygonGroupID(MaterialIndex);

				FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(LODIndex, DrawCallIndex);
				Info.MaterialIndex = MaterialIndex;
				StaticMesh->GetSectionInfoMap().Set(LODIndex, DrawCallIndex, Info);

				int32 TriangleCount = DrawCall.m_uiIndexCount / 3;
				for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
				{
					TArray<FVertexInstanceID> CornerVertexInstanceIDs;
					CornerVertexInstanceIDs.SetNum(3);
					FVertexID CornerVertexIDs[3];
					for (int32 Corner = 0; Corner < 3; ++Corner)
					{
						int32 VertexIndex = LOD.Indices()[DrawCall.m_uiIndexStart + TriangleIndex * 3 + (2 - Corner)];
						const GameEngine9::SVertex& Vertex = LOD.Vertices()[VertexIndex];

						FVertexID VertexID(VertexIndex);
						const FVertexInstanceID& VertexInstanceID = MeshDescription->CreateVertexInstance(VertexID);
						CornerVertexInstanceIDs[Corner] = VertexInstanceID;
						CornerVertexIDs[Corner] = VertexID;

						// tangents
						FVector TangentX(Vertex.m_vTangent.x, Vertex.m_vTangent.y, Vertex.m_vTangent.z);
						FVector TangentY(-Vertex.m_vBinormal.x, -Vertex.m_vBinormal.y, -Vertex.m_vBinormal.z);
						FVector TangentZ(Vertex.m_vNormal.x, Vertex.m_vNormal.y, Vertex.m_vNormal.z);

						VertexInstanceTangents[VertexInstanceID] = (FVector3f)TangentX;
						VertexInstanceNormals[VertexInstanceID] = (FVector3f)TangentZ;
						VertexInstanceBinormalSigns[VertexInstanceID] = GetBasisDeterminantSign(TangentX.GetSafeNormal(), TangentY.GetSafeNormal(), TangentZ.GetSafeNormal());

						// color and branch blend in vertex color
						VertexInstanceColors[VertexInstanceID] = FVector4f(FLinearColor(FColor(Vertex.m_vColor.x * 255, Vertex.m_vColor.y * 255, Vertex.m_vColor.z * 255, Vertex.m_fBlendWeight * 255)));

						// Texcoord setup:
						// 0		Diffuse UV
						// 1		Branch1Pos, Branch1Dir
						// 2		Branch1Weight, RippleWeight
						// 3		Lightmap UV

						// If Branch2 is available
						// 4		Branch2Pos, Branch2Dir
						// 5		Branch2Weight, <Unused>

						// If camera-facing geom is available
						// 4/6		Anchor XY
						// 5/7		Anchor Z, FacingFlag

						// diffuse
						int32 CurrentUV = 0;
						VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vTexCoord.x, Vertex.m_vTexCoord.y));

						// branch1 / ripple
						VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vBranchWind1.x, Vertex.m_vBranchWind1.y));
						VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vBranchWind1.z, Vertex.m_fRippleWeight));

						// lightmap (lightmass can only access 4 uvs)
						VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vLightmapTexCoord.x, Vertex.m_vLightmapTexCoord.y));

						// branch 2
						if (bHasBranch2Data)
						{
							VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vBranchWind2.x, Vertex.m_vBranchWind2.y));
							VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vBranchWind2.z, 0.0f));
						}

						// camera-facing
						if (bHasFacingData)
						{
							VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vAnchor.x, Vertex.m_vAnchor.y));
							VertexInstanceUVs.Set(VertexInstanceID, CurrentUV++, FVector2f(Vertex.m_vAnchor.z, Vertex.m_bCameraFacing ? 1.0f : 0.0f));
						}
					}

					// Insert a polygon into the mesh
					MeshDescription->CreatePolygon(CurrentPolygonGroupID, CornerVertexInstanceIDs);
				}
			}

			//Save the created mesh
			StaticMesh->CommitMeshDescription(LODIndex);
		}

		// replace materials if they've been switched out
		if (OldMaterials.Num() == StaticMesh->GetStaticMaterials().Num())
		{
			StaticMesh->SetStaticMaterials(OldMaterials);
		}

		//Set the Imported version before calling the build
		StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
		StaticMesh->Build();

		// collision objects
		MakeBodyFromCollisionObjects9(StaticMesh, SpeedTree.CollisionObjects());
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, StaticMesh);

	return StaticMesh;
}

USpeedTreeImportData* USpeedTreeImportFactory::GetAutomatedImportOptions(USpeedTreeImportData* ExistingImportData) const
{
	if (AssetImportTask)
	{
		if (USpeedTreeImportData* Options = Cast<USpeedTreeImportData>(AssetImportTask->Options))
		{
			return Options;
		}
	}

	if (ExistingImportData)
	{
		return ExistingImportData;
	}

	USpeedTreeImportData* Options = NewObject<USpeedTreeImportData>(GetTransientPackage(), NAME_None);
	Options->LoadConfig();
	return Options;
}

UObject* USpeedTreeImportFactory::FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	if (FCString::Stricmp(Type, TEXT("ST9")) == 0)
	{
		return FactoryCreateBinary9(InClass, InParent, InName, Flags, Context, Type, Buffer, BufferEnd, Warn, bOutOperationCanceled);
	}
	else if (FCString::Stricmp(Type, TEXT("ST")) == 0)
	{
		return FactoryCreateBinary8(InClass, InParent, InName, Flags, Context, Type, Buffer, BufferEnd, Warn, bOutOperationCanceled);
	}

	return FactoryCreateBinary7(InClass, InParent, InName, Flags, Context, Type, Buffer, BufferEnd, Warn, bOutOperationCanceled);
}

#endif // WITH_SPEEDTREE

#undef LOCTEXT_NAMESPACE
