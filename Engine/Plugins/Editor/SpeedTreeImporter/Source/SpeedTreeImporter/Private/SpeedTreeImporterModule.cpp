// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISpeedTreeImporter.h"

#include "PropertyEditorModule.h"
#include "SpeedTreeImportData.h"

/**
 * SpeedTreeImporter module implementation (private)
 */
class FSpeedTreeImporterModule : public ISpeedTreeImporter
{
public:
	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(USpeedTreeImportData::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FSpeedTreeImportDataDetails::MakeInstance));
	}


	virtual void ShutdownModule() override
	{
	}

};

IMPLEMENT_MODULE(FSpeedTreeImporterModule, SpeedTreeImporter);
