// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * The public interface of the SpeedTreeImporter module
 */
class ISpeedTreeImporter : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to ISpeedTreeImporter
	 *
	 * @return Returns SpeedTreeImporter singleton instance, loading the module on demand if needed
	 */
	static inline ISpeedTreeImporter& Get()
	{
		return FModuleManager::LoadModuleChecked<ISpeedTreeImporter>("SpeedTreeImporter");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("SpeedTreeImporter");
	}


};

