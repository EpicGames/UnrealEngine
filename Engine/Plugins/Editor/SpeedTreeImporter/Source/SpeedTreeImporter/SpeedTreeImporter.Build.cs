// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SpeedTreeImporter : ModuleRules
	{
		public SpeedTreeImporter(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
				    "CoreUObject",
				    "Engine",
				    "Slate",
					"SlateCore",
                    "InputCore",
					"EditorFramework",
                    "UnrealEd",
                    "MainFrame",
                    "MeshDescription",
					"StaticMeshDescription"
                }
				);
				
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SpeedTree");

			if (Target.bUseHeaderUnitsForPch)
			{
				PCHUsage = PCHUsageMode.NoPCHs; // PortableCrt.h in Speedtree uses localtime_s
			}
		}
	}
}
