{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "SpeedTree Importer",
	"Description": "An importer for SpeedTree runtime files.",
	"Category": "Importers",
	"CreatedBy": "IDV, Inc.",
	"CreatedByURL": "https://www.speedtree.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SpeedTreeImporter",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}